{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Dataprep Geometry Operations",
	"Description": "Experimental geometry processing operations usable in the Dataprep Editor.",
	"Category": "Dataprep",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "DataprepGeometryOperations",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "DataprepEditor",
			"Enabled": true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolsetExp",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "ProxyLODPlugin",
			"Enabled": true
		}
	]
}