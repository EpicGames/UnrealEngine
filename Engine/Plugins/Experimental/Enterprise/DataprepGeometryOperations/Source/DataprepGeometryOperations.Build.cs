// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class DataprepGeometryOperations : ModuleRules
	{
		public DataprepGeometryOperations(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "DataprepGO";

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DataprepCore",
					"DataprepLibraries",
					"DynamicMesh",
					"EditorScriptingUtilities",
					"Engine",
					"GeometryCore",
					"MeshConversion",
					"MeshDescription",
					"MeshMergeUtilities",
					"MeshModelingTools",
					"MeshModelingToolsExp",
					"MeshUtilitiesCommon",
					"ModelingOperators",
					"ModelingOperatorsEditorOnly",
					"MeshReductionInterface",
					"StaticMeshDescription",
					"UnrealEd",
				}
			);

			bool bWithProxyLOD = Target.Platform == UnrealTargetPlatform.Win64;
			PrivateDefinitions.Add("WITH_PROXYLOD=" + (bWithProxyLOD ? '1' : '0'));
			if (bWithProxyLOD)
			{
				// For boost:: and TBB:: code
				bUseRTTI = true;

				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"ProxyLODMeshReduction",
					}
				);
			}
		}
	}
}
