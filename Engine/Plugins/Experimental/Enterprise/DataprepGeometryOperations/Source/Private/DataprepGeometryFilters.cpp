// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepGeometryFilters.h"
#include "SelectionSystem/DataprepSelectionSystemStructs.h"
#include "JacketingProcess.h"
#include "GameFramework/Actor.h"

void UDataprepJacketingFilter::ExecuteJacketing(const TArrayView<UObject* const>& InputObjects, TArray<UObject*>& FilteredObjects, const TArrayView<bool>* OutFilterResults) const
{
	TArray<AActor*> InputActors;

	FilteredObjects.Empty();

	for (UObject* Object : InputObjects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			InputActors.Add(Actor);
		}
	}

	if (InputActors.Num() > 0)
	{
		TArray<AActor*> OccludedActors;

		FJacketingOptions JacketingOptions;
		JacketingOptions.Accuracy = GetAccuracy();
		JacketingOptions.MergeDistance = GetMergeDistance();
		JacketingOptions.Target = EJacketingTarget::Level;

		FJacketingProcess::ApplyJacketingOnMeshActors(InputActors, &JacketingOptions, OccludedActors, true);

		FilteredObjects.Append(OccludedActors);
	}

	if (OutFilterResults != nullptr)
	{
		check(OutFilterResults->Num() >= InputObjects.Num());

		for (int ObjectIndex = 0; ObjectIndex < InputObjects.Num(); ++ObjectIndex)
		{
			bool& bFiltered = (*OutFilterResults)[ObjectIndex];
			bFiltered = (INDEX_NONE != FilteredObjects.Find(InputObjects[ObjectIndex]));

			if (IsExcludingResult())
			{
				bFiltered = !bFiltered;
			}
		}
	}
}

float UDataprepJacketingFilter::GetAccuracy() const
{
	return VoxelPrecision;
}

float UDataprepJacketingFilter::GetMergeDistance() const
{
	return GapMaxDiameter;
}

void UDataprepJacketingFilter::SetAccuracy(float NewAccuracy)
{
	if (VoxelPrecision != NewAccuracy)
	{
		Modify();
		VoxelPrecision = NewAccuracy;
	}
}

void UDataprepJacketingFilter::SetMergeDistance(float NewMergeDistance)
{
	if (GapMaxDiameter != NewMergeDistance)
	{
		Modify();
		GapMaxDiameter = NewMergeDistance;
	}
}

TArray<UObject*> UDataprepJacketingFilter::FilterObjects(const TArrayView<UObject* const>& Objects) const
{
	TArray<UObject*> FilteredObjects;

	ExecuteJacketing(Objects, FilteredObjects, nullptr);

	return FilteredObjects;
}

void UDataprepJacketingFilter::FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const
{
	TArray<UObject*> FilteredObjects;
	TArray<bool> FilterResults;
	FilterResults.Init(false, InObjects.Num());
	TArrayView<bool> FilterResultsView(FilterResults);

	ExecuteJacketing(InObjects, FilteredObjects, &FilterResultsView);

	for (int Index = 0; Index < InObjects.Num(); ++Index)
	{
		FDataprepSelectionInfo& SelectionInfo = OutFilterResults[Index];
		SelectionInfo.bHasPassFilter = FilterResultsView[Index];
		SelectionInfo.bWasDataFetchedAndCached = false;
	}
}

void UDataprepJacketingFilter::FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const
{
	TArray<UObject*> FilteredObjects;
	ExecuteJacketing(InObjects, FilteredObjects, &OutFilterResults);
}

FText UDataprepJacketingFilter::GetFilterCategoryText() const
{
	return NSLOCTEXT("DataprepJacketingFilter", "JacketingFilterCategory", "Condition");
}
