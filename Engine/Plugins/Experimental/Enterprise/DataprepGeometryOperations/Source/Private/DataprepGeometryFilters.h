// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SelectionSystem/DataprepFilter.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "DataprepGeometryFilters.generated.h"

UCLASS(BlueprintType, NotBlueprintable, HideCategories = (Filter), Meta = (DisplayName="Jacketing", ToolTip = "Apply mesh jacketing to selected objects"))
class UDataprepJacketingFilter : public UDataprepFilterNoFetcher
{
	GENERATED_BODY()

public:
	UDataprepJacketingFilter() {}

	float GetAccuracy() const;
	float GetMergeDistance() const;

	void SetAccuracy(float NewAccuracy);
	void SetMergeDistance(float NewMergeDistance);

	//~ Begin UDataprepFilter Interface
	virtual TArray<UObject*> FilterObjects(const TArrayView<UObject* const>& Objects) const override;
	virtual void FilterAndGatherInfo(const TArrayView<UObject*>& InObjects, const TArrayView<FDataprepSelectionInfo>& OutFilterResults) const override;
	virtual void FilterAndStoreInArrayView(const TArrayView<UObject*>& InObjects, const TArrayView<bool>& OutFilterResults) const override;
	virtual bool IsThreadSafe() const override { return true; }
	virtual FText GetFilterCategoryText() const override;
	virtual bool UseFetchers() const { return false; }
	//~ End UDataprepFilter Interface

private:
	void ExecuteJacketing(const TArrayView<UObject* const>& InputObjects, TArray<UObject*>& FilteredObjects, const TArrayView<bool>* OutFilterResults) const;

private:
	/** Accuracy of the distance field approximation, in cm. */
	UPROPERTY(EditAnywhere, Category = JacketingFilter, meta = (Units = cm, UIMin = "0.1", UIMax = "100", ClampMin = "0"))
	float VoxelPrecision = 3.0f;

	/** Merge distance used to fill gap, in cm. */
	UPROPERTY(EditAnywhere, Category = JacketingFilter, meta = (Units = cm, UIMin = "0.1", UIMax = "100", ClampMin = "0"))
	float GapMaxDiameter = 4.0f;
};
