// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepGeometryOperations.h"

#include "DataprepOperationsLibraryUtil.h"
#include "DataprepAssetUserData.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMeshToMeshDescription.h"
#include "DynamicMeshEditor.h"
#include "Components/StaticMeshComponent.h"
#include "CuttingOps/PlaneCutOp.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "IDataprepProgressReporter.h"
#include "IMeshReductionInterfaces.h"
#include "IMeshReductionManagerModule.h"
#include "Modules/ModuleManager.h"
#include "MeshAdapterTransforms.h"
#include "MeshDescriptionAdapter.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "PhysicsEngine/BodySetup.h"
#include "UObject/UObjectIterator.h"

DEFINE_LOG_CATEGORY(LogDataprepGeometryOperations);

#define LOCTEXT_NAMESPACE "DatasmithEditingOperationsExperimental"

#ifdef LOG_TIME
namespace DataprepGeometryOperationsTime
{
	typedef TFunction<void(FText)> FLogFunc;

	class FTimeLogger
	{
	public:
		FTimeLogger(const FString& InText, FLogFunc&& InLogFunc)
			: StartTime( FPlatformTime::Cycles64() )
			, Text( InText )
			, LogFunc(MoveTemp(InLogFunc))
		{
			UE_LOGF( LogDataprep, Log, "%ls ...", *Text );
		}

		~FTimeLogger()
		{
			// Log time spent to import incoming file in minutes and seconds
			double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

			int ElapsedMin = int(ElapsedSeconds / 60.0);
			ElapsedSeconds -= 60.0 * (double)ElapsedMin;
			FText Msg = FText::Format( LOCTEXT("DataprepOperation_LogTime", "{0} took {1} min {2} s."), FText::FromString( Text ), ElapsedMin, FText::FromString( FString::Printf( TEXT("%.3f"), ElapsedSeconds ) ) );
			LogFunc( Msg );
		}

	private:
		uint64 StartTime;
		FString Text;
		FLogFunc LogFunc;
	};
}
#endif

void UDataprepRemeshOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepGeometryOperationsTime::FTimeLogger TimeLogger( TEXT("RemeshMesh"), [&]( FText Text) { this->LogInfo( Text ); });
#endif
	
	TArray<UObject*> ModifiedStaticMeshes;


	TSet<UStaticMesh*> SelectedMeshes = DataprepOperationsLibraryUtil::GetSelectedMeshes(InContext.Objects);

	// Apply remesher

	for (UStaticMesh* StaticMesh : SelectedMeshes)
	{
		if (!StaticMesh)
		{
			continue;
		}
		
		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);

		TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh = MakeShared<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe>();
		FMeshDescriptionToDynamicMesh MeshDescriptionToDynamicMesh;
		MeshDescriptionToDynamicMesh.Convert(MeshDescription, *OriginalMesh);

		UE::Geometry::FRemeshMeshOp Op;

		Op.RemeshType = RemeshType;
		Op.bCollapses = true;
		Op.bDiscardAttributes = bDiscardAttributes;
		Op.bFlips = true;
		Op.bPreserveSharpEdges = true;
		Op.MeshBoundaryConstraint = (EEdgeRefineFlags)MeshBoundaryConstraint;
		Op.GroupBoundaryConstraint = (EEdgeRefineFlags)GroupBoundaryConstraint;
		Op.MaterialBoundaryConstraint = (EEdgeRefineFlags)MaterialBoundaryConstraint;
		Op.bPreventNormalFlips = true;
		Op.bReproject = true;
		Op.bSplits = true;
		Op.RemeshIterations = RemeshIterations;
		Op.SmoothingStrength = SmoothingStrength;
		Op.SmoothingType = ERemeshSmoothingType::MeanValue;

		TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> OriginalMeshSpatial = MakeShared<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe>(OriginalMesh.Get(), true);

		double InitialMeshArea = 0;
		for (int tid : OriginalMesh->TriangleIndicesItr())
		{
			InitialMeshArea += OriginalMesh->GetTriArea(tid);
		}

		double TargetTriArea = InitialMeshArea / (double)TargetTriangleCount;
		double EdgeLen = UE::Geometry::TriangleUtil::EquilateralEdgeLengthForArea(TargetTriArea);
		Op.TargetEdgeLength = (double)FMath::RoundToInt(EdgeLen*100.0) / 100.0;

		Op.OriginalMesh = OriginalMesh;
		Op.OriginalMeshSpatial = OriginalMeshSpatial;

		FProgressCancel Progress;
		Op.CalculateResult(&Progress);

		// Update the static mesh with result

		FDynamicMeshToMeshDescription DynamicMeshToMeshDescription;

		// full conversion if normal topology changed or faces were inverted
		TUniquePtr<UE::Geometry::FDynamicMesh3> ResultMesh = Op.ExtractResult();
		DynamicMeshToMeshDescription.Convert(ResultMesh.Get(), *MeshDescription);

		UStaticMesh::FCommitMeshDescriptionParams Params;
		Params.bMarkPackageDirty = false;
		Params.bUseHashAsGuid = true;
		StaticMesh->CommitMeshDescription(0, Params);

		ModifiedStaticMeshes.Add( StaticMesh );
	}

	if(ModifiedStaticMeshes.Num() > 0)
	{
		AssetsModified( MoveTemp( ModifiedStaticMeshes ) );
	}
}

void UDataprepSimplifyMeshOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepGeometryOperationsTime::FTimeLogger TimeLogger( TEXT("SimplifyMesh"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	// Execute operation
	TArray<UObject*> ModifiedStaticMeshes;

	TSet<UStaticMesh*> SelectedMeshes = DataprepOperationsLibraryUtil::GetSelectedMeshes(InContext.Objects);

	IMeshReductionManagerModule& MeshReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");

	for (UStaticMesh* StaticMesh : SelectedMeshes)
	{
		if (!StaticMesh)
		{
			continue;
		}
		
		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);

		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
		FMeshDescriptionToDynamicMesh MeshDescriptionToDynamicMesh;
		MeshDescriptionToDynamicMesh.Convert(MeshDescription, *OriginalMesh);

		TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe> OriginalMeshSpatial = MakeShared<UE::Geometry::FDynamicMeshAABBTree3, ESPMode::ThreadSafe>(OriginalMesh.Get(), true);

		TSharedPtr<FMeshDescription, ESPMode::ThreadSafe> OriginalMeshDescription = MakeShared<FMeshDescription, ESPMode::ThreadSafe>(*StaticMesh->GetMeshDescription(0));

		UE::Geometry::FSimplifyMeshOp Op;

		Op.bDiscardAttributes = bDiscardAttributes;
		Op.bPreventNormalFlips = true;
		Op.bPreserveSharpEdges = true;
		Op.bReproject = false;
		Op.SimplifierType = ESimplifyType::UEStandard;
//		Op.TargetCount = TargetCount;
//		Op.TargetEdgeLength = TargetEdgeLength;
		Op.TargetMode = ESimplifyTargetType::Percentage;
		Op.TargetPercentage = TargetPercentage;
		Op.MeshBoundaryConstraint = (EEdgeRefineFlags)MeshBoundaryConstraint;
		Op.GroupBoundaryConstraint = (EEdgeRefineFlags)GroupBoundaryConstraint;
		Op.MaterialBoundaryConstraint = (EEdgeRefineFlags)MaterialBoundaryConstraint;
		Op.OriginalMeshDescription = OriginalMeshDescription;
		Op.OriginalMesh = OriginalMesh;
		Op.OriginalMeshSpatial = OriginalMeshSpatial;

		Op.MeshReduction = MeshReductionModule.GetStaticMeshReductionInterface();

		FProgressCancel Progress;
		Op.CalculateResult(&Progress);

		// Update the static mesh with result

		FDynamicMeshToMeshDescription DynamicMeshToMeshDescription;

		// Convert back to mesh description
		TUniquePtr<UE::Geometry::FDynamicMesh3> ResultMesh = Op.ExtractResult();
		DynamicMeshToMeshDescription.Convert(ResultMesh.Get(), *MeshDescription);

		UStaticMesh::FCommitMeshDescriptionParams Params;
		Params.bMarkPackageDirty = false;
		Params.bUseHashAsGuid = true;
		StaticMesh->CommitMeshDescription(0, Params);

		ModifiedStaticMeshes.Add( StaticMesh );
	}

	if(ModifiedStaticMeshes.Num() > 0)
	{
		AssetsModified( MoveTemp( ModifiedStaticMeshes ) );
	}
}

void UDataprepBakeTransformOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepGeometryOperationsTime::FTimeLogger TimeLogger( TEXT("BakeTransform"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	TArray<UObject*> ModifiedStaticMeshes;

	TArray<UStaticMeshComponent*> StaticMeshComponents;
	TArray<AActor*> ComponentActors;

	for (UObject* Object : InContext.Objects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			TInlineComponentArray<UStaticMeshComponent*> Components(Actor);
			for (UStaticMeshComponent* Component : Components)
			{
				StaticMeshComponents.Add(Component);
				ComponentActors.Add(Actor);
			}
		}
	}

	bool bSharesSources = false;
	TArray<int> MapToFirstOccurrences;
	MapToFirstOccurrences.SetNumUninitialized(StaticMeshComponents.Num());
	for (int32 ComponentIdx = 0; ComponentIdx < StaticMeshComponents.Num(); ComponentIdx++)
	{
		MapToFirstOccurrences[ComponentIdx] = -1;
	}
	for (int32 ComponentIdx = 0; ComponentIdx < StaticMeshComponents.Num(); ComponentIdx++)
	{
		if (MapToFirstOccurrences[ComponentIdx] >= 0) // already mapped
		{
			continue;
		}

		MapToFirstOccurrences[ComponentIdx] = ComponentIdx;

		UStaticMeshComponent* Component = StaticMeshComponents[ComponentIdx];
		for (int32 VsIdx = ComponentIdx + 1; VsIdx < StaticMeshComponents.Num(); VsIdx++)
		{
			UStaticMeshComponent* OtherComponent = StaticMeshComponents[VsIdx];

			const UStaticMesh* StaticMesh = Component->GetStaticMesh();
			const UStaticMesh* OtherStaticMesh = OtherComponent->GetStaticMesh();
			if ( StaticMesh && StaticMesh == OtherStaticMesh )
			{
				bSharesSources = true;
				MapToFirstOccurrences[VsIdx] = ComponentIdx;
			}
		}
	}

	TArray<UE::Geometry::FTransformSRT3d> BakedTransforms;
	for (int32 ComponentIdx = 0; ComponentIdx < StaticMeshComponents.Num(); ComponentIdx++)
	{
		UE::Geometry::FTransformSRT3d ComponentToWorld(StaticMeshComponents[ComponentIdx]->GetComponentTransform());
		UE::Geometry::FTransformSRT3d ToBakePart = UE::Geometry::FTransformSRT3d::Identity();
		UE::Geometry::FTransformSRT3d NewWorldPart = ComponentToWorld;

		if (MapToFirstOccurrences[ComponentIdx] < ComponentIdx)
		{
			ToBakePart = BakedTransforms[MapToFirstOccurrences[ComponentIdx]];
			BakedTransforms.Add(ToBakePart);
			// Try to invert baked transform
			NewWorldPart = UE::Geometry::FTransformSRT3d(
				NewWorldPart.GetRotation() * ToBakePart.GetRotation().Inverse(),
				NewWorldPart.GetTranslation(),
				NewWorldPart.GetScale() * UE::Geometry::FTransformSRT3d::GetSafeScaleReciprocal(ToBakePart.GetScale())
			);
			NewWorldPart.SetTranslation(NewWorldPart.GetTranslation() - NewWorldPart.TransformVector(ToBakePart.GetTranslation()));
		}
		else
		{
			if (bBakeRotation)
			{
				ToBakePart.SetRotation(ComponentToWorld.GetRotation());
				NewWorldPart.SetRotation(UE::Geometry::FQuaterniond::Identity());
			}
			FVector3d ScaleVec = ComponentToWorld.GetScale();

			FVector3d AbsScales(FMathd::Abs(ScaleVec.X), FMathd::Abs(ScaleVec.Y), FMathd::Abs(ScaleVec.Z));
			double RemainingUniformScale = AbsScales.X;
			{
				FVector3d Dists;
				for (int SubIdx = 0; SubIdx < 3; SubIdx++)
				{
					int OtherA = (SubIdx + 1) % 3;
					int OtherB = (SubIdx + 2) % 3;
					Dists[SubIdx] = FMathd::Abs(AbsScales[SubIdx] - AbsScales[OtherA]) + FMathd::Abs(AbsScales[SubIdx] - AbsScales[OtherB]);
				}
				int BestSubIdx = 0;
				for (int CompareSubIdx = 1; CompareSubIdx < 3; CompareSubIdx++)
				{
					if (Dists[CompareSubIdx] < Dists[BestSubIdx])
					{
						BestSubIdx = CompareSubIdx;
					}
				}
				RemainingUniformScale = AbsScales[BestSubIdx];
				if (RemainingUniformScale <= FLT_MIN)
				{
					RemainingUniformScale = UE::Geometry::MaxAbsElement(AbsScales);
				}
			}
			switch (BakeScale)
			{
			case EBakeScaleMethod::BakeFullScale:
				ToBakePart.SetScale(ScaleVec);
				NewWorldPart.SetScale(FVector3d::One());
				break;
			case EBakeScaleMethod::BakeNonuniformScale:
				check(RemainingUniformScale > FLT_MIN); // avoid baking a ~zero scale
				ToBakePart.SetScale(ScaleVec / RemainingUniformScale);
				NewWorldPart.SetScale(FVector3d(RemainingUniformScale, RemainingUniformScale, RemainingUniformScale));
				break;
			case EBakeScaleMethod::DoNotBakeScale:
				break;
			default:
				check(false); // must explicitly handle all cases
			}

			UStaticMeshComponent* Component = StaticMeshComponents[ComponentIdx];

			UStaticMesh* StaticMesh = Component->GetStaticMesh();

			FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);

			FMeshDescriptionEditableTriangleMeshAdapter EditableMeshDescAdapter(MeshDescription);

			// Do this part within the commit because we have the MeshDescription already computed
			if (bRecenterPivot)
			{
				FBox BBox = MeshDescription->ComputeBoundingBox();
				FVector3d Center(BBox.GetCenter());
				UE::Geometry::FFrame3d LocalFrame(Center);
				ToBakePart.SetTranslation(ToBakePart.GetTranslation() - Center);
				NewWorldPart.SetTranslation(NewWorldPart.GetTranslation() + NewWorldPart.TransformVector(Center));
			}

			MeshAdapterTransforms::ApplyTransform(EditableMeshDescAdapter, ToBakePart);

			ScaleVec = ToBakePart.GetScale();
			if (ScaleVec.X * ScaleVec.Y * ScaleVec.Z < 0)
			{
				MeshDescription->ReverseAllPolygonFacing();
			}

			UStaticMesh::FCommitMeshDescriptionParams Params;
			Params.bMarkPackageDirty = false;
			Params.bUseHashAsGuid = true;
			StaticMesh->CommitMeshDescription(0, Params);

			ModifiedStaticMeshes.Add(StaticMesh);

			BakedTransforms.Add(ToBakePart);
		}

		UStaticMeshComponent* Component = StaticMeshComponents[ComponentIdx];
		Component->SetWorldTransform((FTransform)NewWorldPart);
		ComponentActors[ComponentIdx]->MarkComponentsRenderStateDirty();
	}
}

void UDataprepWeldEdgesOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepGeometryOperationsTime::FTimeLogger TimeLogger( TEXT("WeldEdges"), [&]( FText Text) { this->LogInfo( Text ); });
#endif

	TArray<UObject*> ModifiedStaticMeshes;

	TSet<UStaticMesh*> SelectedMeshes = DataprepOperationsLibraryUtil::GetSelectedMeshes(InContext.Objects);

	for (UStaticMesh* StaticMesh : SelectedMeshes)
	{
		if (!StaticMesh)
		{
			continue;
		}
		
		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);

		TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> TargetMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
		FMeshDescriptionToDynamicMesh MeshDescriptionToDynamicMesh;
		MeshDescriptionToDynamicMesh.Convert(MeshDescription, *TargetMesh);

		UE::Geometry::FMergeCoincidentMeshEdges Merger(TargetMesh.Get());
		Merger.MergeSearchTolerance = Tolerance;
		Merger.OnlyUniquePairs = bOnlyUnique;
		
		if (Merger.Apply() == false)
		{
			continue;
		}

		if (TargetMesh->CheckValidity(true, UE::Geometry::EValidityCheckFailMode::ReturnOnly) == false)
		{
			continue; // Target mesh is invalid
		}

		FDynamicMeshToMeshDescription DynamicMeshToMeshDescription;

		// full conversion if normal topology changed or faces were inverted
		DynamicMeshToMeshDescription.Convert(TargetMesh.Get(), *MeshDescription);

		UStaticMesh::FCommitMeshDescriptionParams Params;
		Params.bMarkPackageDirty = false;
		Params.bUseHashAsGuid = true;
		StaticMesh->CommitMeshDescription(0, Params);

		ModifiedStaticMeshes.Add(StaticMesh);
	}

	if(ModifiedStaticMeshes.Num() > 0)
	{
		AssetsModified( MoveTemp( ModifiedStaticMeshes ) );
	}
}

void UDataprepPlaneCutOperation::OnExecution_Implementation(const FDataprepContext& InContext)
{
#ifdef LOG_TIME
	DataprepEditingOperationTime::FTimeLogger TimeLogger(TEXT("PlaneCutOperation"), [&](FText Text) { this->LogInfo(Text); });
#endif

	TArray<UStaticMeshComponent*> StaticMeshComponents;
	TArray<AActor*> ComponentActors;

	TSet<UStaticMesh*> StaticMeshesSet;

	for (UObject* Object : InContext.Objects)
	{
		if (AActor* Actor = Cast< AActor >(Object))
		{
			TInlineComponentArray<UStaticMeshComponent*> Components(Actor);
			for (UStaticMeshComponent* Component : Components)
			{
				if (Component && Component->GetStaticMesh())
				{
					StaticMeshComponents.Add(Component);
					ComponentActors.Add(Actor);
				}
			}
		}
		else if (UStaticMesh* StaticMesh = Cast< UStaticMesh >(Object))
		{
			StaticMeshesSet.Add(StaticMesh);
		}
	}

	// Remove the static meshes that are also referenced in the static mesh components (components take precedence)
	for (UStaticMeshComponent* MeshComp : StaticMeshComponents)
	{
		if (UStaticMesh* StaticMesh = MeshComp->GetStaticMesh())
		{
			if (StaticMeshesSet.Contains(StaticMesh))
			{
				StaticMeshesSet.Remove(StaticMesh);

				if (StaticMeshesSet.Num() == 0)
				{
					break;
				}
			}
		}
	}

	TArray<UObject*> ModifiedStaticMeshes;

	TArray<UStaticMeshComponent*> CutawayStaticMeshes;

	// Cut static meshes
	
	if (StaticMeshesSet.Num() > 0)
	{
		TArray<UStaticMesh*> StaticMeshes = StaticMeshesSet.Array();
		TArray<TArray<UStaticMeshComponent*>> ReferencingComponentsToUpdate;
		TArray<FTransform> CutPlaneTransforms;

		ReferencingComponentsToUpdate.Init(TArray<UStaticMeshComponent*>(), StaticMeshes.Num());

		// Use identity transforms when cutting static meshes
		CutPlaneTransforms.Init(FTransform(), StaticMeshes.Num());

		for (TObjectIterator<UStaticMeshComponent> It; It; ++It)
		{
			if (UStaticMesh* StaticMesh = It->GetStaticMesh())
			{
				if (StaticMeshesSet.Contains(StaticMesh))
				{
					const int32 MeshIndex = StaticMeshes.Find(StaticMesh);
					check(MeshIndex != INDEX_NONE);
					ReferencingComponentsToUpdate[MeshIndex].Add(*It);
				}
			}
		}

		PerformCutting(false, StaticMeshes, CutPlaneTransforms, ReferencingComponentsToUpdate, ModifiedStaticMeshes, CutawayStaticMeshes);
	}

	// Cut static mesh components

	if (StaticMeshComponents.Num() > 0)
	{
		TArray<UStaticMesh*> StaticMeshes;
		TArray<TArray<UStaticMeshComponent*>> ReferencingComponentsToUpdate;
		TArray<FTransform> CutPlaneTransforms;

		StaticMeshes.Reserve(StaticMeshComponents.Num());
		ReferencingComponentsToUpdate.Reserve(StaticMeshComponents.Num());
		CutPlaneTransforms.Reserve(StaticMeshComponents.Num());

		for (UStaticMeshComponent* Component : StaticMeshComponents)
		{
			if (UStaticMesh* StaticMesh = Component->GetStaticMesh())
			{
				StaticMeshes.Add(StaticMesh);
				ReferencingComponentsToUpdate.AddDefaulted_GetRef().Add(Component);
				CutPlaneTransforms.Add(Component->GetComponentTransform());
			}
		}

		PerformCutting(true, StaticMeshes, CutPlaneTransforms, ReferencingComponentsToUpdate, ModifiedStaticMeshes, CutawayStaticMeshes);
	}

	// Remove actors that have no valid static mesh as a result of the cutting
	TSet<UObject*> ActorsToRemove;
	for (UStaticMeshComponent* CutoffComponent : CutawayStaticMeshes)
	{
		if (AActor* Owner = CutoffComponent->GetOwner())
		{
			TInlineComponentArray<UStaticMeshComponent*> ComponentArray;
			Owner->GetComponents(ComponentArray, true);

			bool bMeshActorIsValid = false;

			for(UStaticMeshComponent* MeshComponent : ComponentArray)
			{
				if (!MeshComponent || MeshComponent == CutoffComponent)
				{
					continue; // We know this component does not have a mesh
				}

				if (MeshComponent->GetStaticMesh() && MeshComponent->GetStaticMesh()->GetSourceModels().Num() > 0)
				{
					bMeshActorIsValid = true;
					break;
				}
			}

			if (!bMeshActorIsValid)
			{
				ActorsToRemove.Add(Owner);
			}
			else
			{
				CutoffComponent->DestroyComponent();
			}
		}
	}

	DeleteObjects(ActorsToRemove.Array());

	if (ModifiedStaticMeshes.Num() > 0)
	{
		AssetsModified(MoveTemp(ModifiedStaticMeshes));
	}
}

TUniquePtr<FDynamicMesh3> UDataprepPlaneCutOperation::CutStaticMesh(const FTransform& InTransform, const UStaticMesh* InStaticMesh)
{
	const FMeshDescription* MeshDescription = InStaticMesh->GetMeshDescription(0);

	TSharedPtr<FDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>();
	FMeshDescriptionToDynamicMesh MeshDescriptionToDynamicMesh;
	MeshDescriptionToDynamicMesh.Convert(MeshDescription, *OriginalMesh);

	OriginalMesh->EnableAttributes();
	UE::Geometry::TDynamicMeshScalarTriangleAttribute<int>* SubObjectIDs = new UE::Geometry::TDynamicMeshScalarTriangleAttribute<int>(OriginalMesh.Get());
	SubObjectIDs->Initialize(0);
	OriginalMesh->Attributes()->AttachAttribute(UE::Geometry::FPlaneCutOp::ObjectIndexAttribute, SubObjectIDs);

	// Store a UV scale based on the original mesh bounds (we don't want to recompute this between cuts b/c we want consistent UV scale)
	const float MeshUVScaleFactor = 1.0 / OriginalMesh->GetBounds().MaxDim();

	TUniquePtr<UE::Geometry::FDynamicMeshOperator> CutOp = MakeNewOperator(InTransform, OriginalMesh, MeshUVScaleFactor);

	FProgressCancel Progress;
	CutOp->CalculateResult(&Progress);

	return CutOp->ExtractResult();
}

void UDataprepPlaneCutOperation::PerformCutting(
	bool bKeepOriginalMesh, 
	TArray<UStaticMesh*>& InStaticMeshes, 
	const TArray<FTransform>& InCutPlaneTransforms, 
	TArray<TArray<UStaticMeshComponent*>>& InReferencingComponentsToUpdate,
	TArray<UObject*>& OutModifiedStaticMeshes, 
	TArray<UStaticMeshComponent*>& OutCutawayMeshes)
{
	TArray<TUniquePtr<FDynamicMesh3>> Results;
	Results.SetNum(InStaticMeshes.Num());

	ParallelFor( Results.Num(), [this, &Results, &InCutPlaneTransforms, &InStaticMeshes](int32 Index)
	{
		Results[Index] = CutStaticMesh(InCutPlaneTransforms[Index], InStaticMeshes[Index]);
	});

	TArray<TArray<TUniquePtr<FDynamicMesh3>>> AllSplitMeshes;
	AllSplitMeshes.SetNum(InStaticMeshes.Num());

	for (int OrigMeshIdx = 0; OrigMeshIdx < InStaticMeshes.Num(); OrigMeshIdx++)
	{
		FDynamicMesh3* UseMesh = Results[OrigMeshIdx].Get();
		check(UseMesh != nullptr);

		// Check if mesh was entirely cut away
		if (UseMesh->TriangleCount() == 0)
		{
			TArray<UStaticMeshComponent*>& ComponentsToUpdate = InReferencingComponentsToUpdate[OrigMeshIdx];

			for (UStaticMeshComponent* Component : ComponentsToUpdate)
			{
				Component->SetStaticMesh(nullptr);
				Component->MarkRenderStateDirty();
				OutCutawayMeshes.Add(Component);
			}
			continue;
		}

		UStaticMesh* StaticMesh = InStaticMeshes[OrigMeshIdx];

		if (bExportSeparatePieces)
		{
			// Export separated pieces as new mesh assets

			UE::Geometry::TDynamicMeshScalarTriangleAttribute<int>* SubMeshIDs =
				static_cast<UE::Geometry::TDynamicMeshScalarTriangleAttribute<int>*>(UseMesh->Attributes()->GetAttachedAttribute(
					UE::Geometry::FPlaneCutOp::ObjectIndexAttribute));
			TArray<TUniquePtr<UE::Geometry::FDynamicMesh3>>& SplitMeshes = AllSplitMeshes[OrigMeshIdx];
			bool bWasSplit = UE::Geometry::FDynamicMeshEditor::SplitMesh(UseMesh, SplitMeshes, false, [SubMeshIDs](int TID)
			{
				return SubMeshIDs->GetValue(TID);
			});
			if (bWasSplit)
			{
				// Split mesh did something but has no meshes in the output array??
				if (!ensure(SplitMeshes.Num() > 0))
				{
					continue;
				}

				if (SplitMeshes.Num() > 1)
				{
					TArray<UStaticMeshComponent*>& ComponentsToUpdate = InReferencingComponentsToUpdate[OrigMeshIdx];

					for (UStaticMeshComponent* ComponentTarget : ComponentsToUpdate)
					{
						// Build array of materials from the original
						TArray<UMaterialInterface*> Materials;
						for (int MaterialIdx = 0, NumMaterials = ComponentTarget->GetNumMaterials(); MaterialIdx < StaticMesh->GetStaticMaterials().Num(); MaterialIdx++)
						{
							Materials.Add(ComponentTarget->GetMaterial(MaterialIdx));
						}

						// Add all the additional meshes
						for (int AddMeshIdx = 1; AddMeshIdx < SplitMeshes.Num(); AddMeshIdx++)
						{
							FTransform Transform = ComponentTarget->GetComponentTransform();

							FDynamicMesh3* Mesh = SplitMeshes[AddMeshIdx].Get();

							TUniquePtr<FMeshDescription> MeshDescription = MakeUnique<FMeshDescription>();
							FStaticMeshAttributes Attributes(*MeshDescription);
							Attributes.Register();

							FDynamicMeshToMeshDescription Converter;
							Converter.Convert(Mesh, *MeshDescription);

							// Add new actor
							AStaticMeshActor* NewActor = Cast<AStaticMeshActor>(CreateActor(AStaticMeshActor::StaticClass(), FString()));
							check(NewActor);

							AActor* OriginalActor = ComponentTarget->GetOwner<AActor>();
							check(OriginalActor);

							NewActor->SetActorLabel(OriginalActor->GetActorLabel() + "_Below");

							const FString NewMeshName = StaticMesh->GetName() + "_Below";

							// Create new mesh component and set as root of NewActor.
							UStaticMeshComponent* NewMeshComponent = FinalizeStaticMeshActor(NewActor, NewMeshName, MeshDescription.Get(), Materials.Num(), StaticMesh);
							
							NewMeshComponent->SetMobility(ComponentTarget->Mobility);
							NewMeshComponent->SetVisibility(ComponentTarget->IsVisible());
							NewMeshComponent->ComponentTags = ComponentTarget->ComponentTags;
							
							// Add dataprep user data
							if (ComponentTarget->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()) && 
								NewMeshComponent->GetClass()->ImplementsInterface(UInterface_AssetUserData::StaticClass()))
							{
								IInterface_AssetUserData* OriginalAssetUserDataInterface = Cast< IInterface_AssetUserData >(ComponentTarget);
								IInterface_AssetUserData* NewAssetUserDataInterface = Cast< IInterface_AssetUserData >(NewMeshComponent);

								if (OriginalAssetUserDataInterface && NewAssetUserDataInterface)
								{
									if (const TArray<UAssetUserData*>* UserDataArray = OriginalAssetUserDataInterface->GetAssetUserDataArray())
									{
										for ( UAssetUserData* UserData : *UserDataArray )
										{
											if (UserData)
											{
												UObject* NewUserDataOuter = (UserData->GetOuter() == ComponentTarget) ? NewMeshComponent : UserData->GetOuter();
												UAssetUserData* NewUserData = DuplicateObject<UAssetUserData>(UserData, NewUserDataOuter);
												NewAssetUserDataInterface->AddAssetUserData(NewUserData);
											}
										}
									}
								}
							}

							NewActor->Tags = OriginalActor->Tags;
							NewActor->Layers = OriginalActor->Layers;

							// Keep the newly created actor at the same level in hierarchy as the original one
							AActor* Parent = OriginalActor->GetAttachParentActor();
							if (Parent != nullptr)
							{
								NewActor->AttachToActor(Parent, FAttachmentTransformRules::KeepWorldTransform);
							}

							// Configure transform and materials of new component
							NewMeshComponent->SetWorldTransform((FTransform)Transform);
							for (int MatIdx = 0, NumMats = Materials.Num(); MatIdx < NumMats; MatIdx++)
							{
								NewMeshComponent->SetMaterial(MatIdx, Materials[MatIdx]);
							}
						}
					}
				}
				UseMesh = SplitMeshes[0].Get();
			}
		}

		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);

		if (bKeepOriginalMesh)
		{
			// Create new mesh in order to preserve the original

			EObjectFlags flags = EObjectFlags::RF_Public | EObjectFlags::RF_Standalone;
			UStaticMesh* NewStaticMesh = Cast<UStaticMesh>(CreateAsset(UStaticMesh::StaticClass(), StaticMesh->GetName() + "_PlaneCut"));

			// Initialize the LOD 0 MeshDescription
			NewStaticMesh->SetNumSourceModels(1);
			NewStaticMesh->GetSourceModel(0).BuildSettings.bRecomputeNormals = false;
			NewStaticMesh->GetSourceModel(0).BuildSettings.bRecomputeTangents = true;

			MeshDescription = NewStaticMesh->CreateMeshDescription(0);

			if (StaticMesh->GetBodySetup())
			{
				if (NewStaticMesh->GetBodySetup() == nullptr)
				{
					NewStaticMesh->CreateBodySetup();
				}

				NewStaticMesh->GetBodySetup()->CollisionTraceFlag = StaticMesh->GetBodySetup()->CollisionTraceFlag.GetValue();
			}

			const TArray<FStaticMaterial>& MeshMaterials = StaticMesh->GetStaticMaterials();
			for (const FStaticMaterial& Material : MeshMaterials)
			{
				NewStaticMesh->GetStaticMaterials().Add(Material);
			}

			for (UStaticMeshComponent* Component : InReferencingComponentsToUpdate[OrigMeshIdx])
			{
				Component->SetStaticMesh(NewStaticMesh);
			}

			StaticMesh = NewStaticMesh;
		}

		FDynamicMeshToMeshDescription DynamicMeshToMeshDescription;

		// Full conversion if normal topology changed or faces were inverted
		DynamicMeshToMeshDescription.Convert(UseMesh, *MeshDescription);

		UStaticMesh::FCommitMeshDescriptionParams Params;
		Params.bMarkPackageDirty = false;
		Params.bUseHashAsGuid = true;

		StaticMesh->CommitMeshDescription(0, Params);

		for (UStaticMeshComponent* Component : InReferencingComponentsToUpdate[OrigMeshIdx])
		{
			Component->MarkRenderStateDirty();
		}

		if (!bKeepOriginalMesh)
		{
			OutModifiedStaticMeshes.Add(StaticMesh);
		}
	}
}

TUniquePtr<UE::Geometry::FDynamicMeshOperator> UDataprepPlaneCutOperation::MakeNewOperator(
	const FTransform& InMeshLocalToWorld, 
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> InOriginalMesh,
	float InMeshUVScaleFactor)
{
	TUniquePtr<UE::Geometry::FPlaneCutOp> CutOp = MakeUnique<UE::Geometry::FPlaneCutOp>();
	CutOp->bFillCutHole = bFillCutHole;
	CutOp->bFillSpans = false;

	FTransform LocalToWorld = InMeshLocalToWorld;
	CutOp->SetTransform(LocalToWorld);
	// for all plane computation, change LocalToWorld to not have any zero scale dims
	FVector LocalToWorldScale = LocalToWorld.GetScale3D();
	for (int i = 0; i < 3; i++)
	{
		float DimScale = FMathf::Abs(LocalToWorldScale[i]);
		float Tolerance = KINDA_SMALL_NUMBER;
		if (DimScale < Tolerance)
		{
			LocalToWorldScale[i] = Tolerance * FMathf::SignNonZero(LocalToWorldScale[i]);
		}
	}

	// Default plane normal is Z axis (XY plane)
	const FTransform OrientPlane(FRotator::MakeFromEuler(CutPlaneNormalAngles));
	FVector WorldNormal = OrientPlane.TransformVector(FVector::ZAxisVector);

	if (CutPlaneKeepSide == EPlaneCutKeepSide::Positive)
	{
		WorldNormal *= -1;
	}

	LocalToWorld.SetScale3D(LocalToWorldScale);
	FTransform WorldToLocal = LocalToWorld.Inverse();
	FVector LocalOrigin = WorldToLocal.TransformPosition(CutPlaneOrigin);
	UE::Geometry::FTransformSRT3d W2LForNormal(WorldToLocal);
	FVector LocalNormal = (FVector)W2LForNormal.TransformNormal((FVector3d)WorldNormal);
	FVector BackTransformed = LocalToWorld.TransformVector(LocalNormal);
	float NormalScaleFactor = FVector::DotProduct(BackTransformed, WorldNormal);
	if (NormalScaleFactor >= FLT_MIN)
	{
		NormalScaleFactor = 1.0 / NormalScaleFactor;
	}
	CutOp->LocalPlaneOrigin = (FVector3d)LocalOrigin;
	CutOp->LocalPlaneNormal = (FVector3d)LocalNormal;
	CutOp->OriginalMesh = InOriginalMesh;
	CutOp->bKeepBothHalves = (CutPlaneKeepSide == EPlaneCutKeepSide::Both);
	CutOp->CutPlaneLocalThickness = SpacingBetweenHalves * NormalScaleFactor;
	CutOp->UVScaleFactor = InMeshUVScaleFactor;

	return CutOp;
}

UStaticMeshComponent* UDataprepPlaneCutOperation::FinalizeStaticMeshActor(
	AStaticMeshActor* InActor, 
	const FString& InMeshName, 
	const FMeshDescription* InMeshDescription, 
	int InNumMaterialSlots,
	const UStaticMesh* InOriginalMesh)
{
	check(InMeshDescription != nullptr);

	// create new UStaticMesh object
	EObjectFlags flags = EObjectFlags::RF_Public | EObjectFlags::RF_Standalone;
	UStaticMesh* NewStaticMesh = Cast<UStaticMesh>(CreateAsset(UStaticMesh::StaticClass(), InMeshName));

	// initialize the LOD 0 MeshDescription
	NewStaticMesh->SetNumSourceModels(1);
	NewStaticMesh->GetSourceModel(0).BuildSettings.bRecomputeNormals = false;
	NewStaticMesh->GetSourceModel(0).BuildSettings.bRecomputeTangents = true;

	NewStaticMesh->CreateMeshDescription(0, *InMeshDescription);

	if (InOriginalMesh->GetBodySetup())
	{
		if (NewStaticMesh->GetBodySetup() == nullptr)
		{
			NewStaticMesh->CreateBodySetup();
		}

		NewStaticMesh->GetBodySetup()->CollisionTraceFlag = InOriginalMesh->GetBodySetup()->CollisionTraceFlag.GetValue();
	}

	// add a material slot. Must always have one material slot.
	int AddMaterialCount = FMath::Max(1, InNumMaterialSlots);
	for (int MatIdx = 0; MatIdx < AddMaterialCount; MatIdx++)
	{
		NewStaticMesh->GetStaticMaterials().Add(FStaticMaterial());
	}

	// assuming we have updated the LOD 0 MeshDescription, tell UStaticMesh about this
	NewStaticMesh->CommitMeshDescription(0);

	UStaticMeshComponent* NewMeshComponent = nullptr;

	// if we have a StaticMeshActor we already have a StaticMeshComponent, otherwise we 
	// need to make a new one. Note that if we make a new one it will not be editable in the 
	// Editor because it is not a UPROPERTY...
	AStaticMeshActor* StaticMeshActor = Cast<AStaticMeshActor>(InActor);
	if (StaticMeshActor != nullptr)
	{
		NewMeshComponent = StaticMeshActor->GetStaticMeshComponent();
	}
	else
	{
		// create component
		NewMeshComponent = NewObject<UStaticMeshComponent>(InActor);
		InActor->SetRootComponent(NewMeshComponent);
	}

	// this disconnects the component from various events
	NewMeshComponent->UnregisterComponent();

	// Configure flags of the component. Is this necessary?
	NewMeshComponent->SetMobility(EComponentMobility::Movable);
	NewMeshComponent->bSelectable = true;

	// replace the UStaticMesh in the component
	NewMeshComponent->SetStaticMesh(NewStaticMesh);

	// re-connect the component (?)
	NewMeshComponent->RegisterComponent();

	// if we don't do this, world traces don't hit the mesh
	NewMeshComponent->MarkRenderStateDirty();

	return NewMeshComponent;
}

#undef LOCTEXT_NAMESPACE
