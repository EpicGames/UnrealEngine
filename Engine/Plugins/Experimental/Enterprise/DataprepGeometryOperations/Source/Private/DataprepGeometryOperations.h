// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DataprepOperation.h"

#include "Properties/RemeshProperties.h"
#include "CleaningOps/RemeshMeshOp.h"
#include "CleaningOps/SimplifyMeshOp.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "ModelingOperators.h"
#include "BakeTransformTool.h"

#include "DataprepGeometryOperations.generated.h"

DECLARE_LOG_CATEGORY_EXTERN( LogDataprepGeometryOperations, Log, All );

class AStaticMeshActor;
class IMeshBuilderModule;
class UStaticMesh;
class UStaticMeshComponent;
class UWorld;

UCLASS(Experimental, Category = ObjectOperation, Meta = (DisplayName="Remesh", ToolTip = "Experimental - Remesh input meshes") )
class UDataprepRemeshOperation : public UDataprepEditingOperation
{
	GENERATED_BODY()

public:

	/** Target triangle count */
	UPROPERTY(EditAnywhere, Category = Remeshing)
	int TargetTriangleCount = 1000;

	/** Amount of Vertex Smoothing applied within Remeshing */
	UPROPERTY(EditAnywhere, Category = Remeshing, meta = (DisplayName = "Smoothing Rate", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float SmoothingStrength = 0.25;

	/** Discard UVs and existing normals, allowing the remesher to ignore any UV and normal seams. New per-vertex normals are computed. */
	UPROPERTY(EditAnywhere, Category = Remeshing)
	bool bDiscardAttributes = false;

	/** Remeshing type */
	UPROPERTY(EditAnywhere, Category = Remeshing)
	ERemeshType RemeshType = ERemeshType::Standard;

	/** Number of Remeshing passes */
	UPROPERTY(EditAnywhere, Category = Remeshing, meta = (EditCondition = "RemeshType == ERemeshType::FullPass", UIMin = "0", UIMax = "50", ClampMin = "0", ClampMax = "1000"))
	int RemeshIterations = 20;

	/** Mesh Boundary Constraint Type */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Boundary Constraints", meta = (DisplayName = "Mesh Boundary"))
	EMeshBoundaryConstraint MeshBoundaryConstraint = EMeshBoundaryConstraint::Free;

	/** Group Boundary Constraint Type */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Boundary Constraints", meta = (DisplayName = "Group Boundary"))
	EGroupBoundaryConstraint GroupBoundaryConstraint = EGroupBoundaryConstraint::Free;

	/** Material Boundary Constraint Type */
	UPROPERTY(EditAnywhere, Category = "Remeshing|Boundary Constraints", meta = (DisplayName = "Material Boundary"))
	EMaterialBoundaryConstraint MaterialBoundaryConstraint = EMaterialBoundaryConstraint::Free;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Experimental, Category = ObjectOperation, Meta = (DisplayName="Bake Transform", ToolTip = "Experimental - Bake transform of input meshes") )
class UDataprepBakeTransformOperation : public UDataprepEditingOperation
{
	GENERATED_BODY()

public:

	/** Bake rotation */
	UPROPERTY(EditAnywhere, Category = BakeTransform)
	bool bBakeRotation = true;

	/** Bake scale */
	UPROPERTY(EditAnywhere, Category = BakeTransform)
	EBakeScaleMethod BakeScale = EBakeScaleMethod::BakeNonuniformScale;

	/** Recenter pivot after baking transform */
	UPROPERTY(EditAnywhere, Category = BakeTransform)
	bool bRecenterPivot = false;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Experimental, Category = ObjectOperation, Meta = (DisplayName="Weld Edges", ToolTip = "Experimental - Weld edges of input meshes") )
class UDataprepWeldEdgesOperation : public UDataprepEditingOperation
{
	GENERATED_BODY()

public:

	/** Merge search tolerance */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0.000001", UIMax = "0.01", ClampMin = "0.00000001", ClampMax = "1000.0"))
	float Tolerance;

	/** Apply to only unique pairs */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bOnlyUnique;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UCLASS(Experimental, Category = ObjectOperation, Meta = (DisplayName="Simplify Mesh", ToolTip = "Experimental - Simplify input meshes") )
class UDataprepSimplifyMeshOperation : public UDataprepEditingOperation
{
	GENERATED_BODY()

public:

	/** Target percentage of original triangle count */
	UPROPERTY(EditAnywhere, Category = Options, meta = (UIMin = "0", UIMax = "100"))
	int TargetPercentage = 50;

	/** Discard UVs and existing normals, allowing the simplifier to ignore any UV and normal seams. New per-vertex normals are computed. */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bDiscardAttributes = false;

	/** Mesh Boundary Constraint Type */
	UPROPERTY(EditAnywhere, Category = "SimplifyMesh|Boundary Constraints", meta = (DisplayName = "Mesh Boundary"))
	EMeshBoundaryConstraint MeshBoundaryConstraint = EMeshBoundaryConstraint::Free;

	/** Group Boundary Constraint Type */
	UPROPERTY(EditAnywhere, Category = "SimplifyMesh|Boundary Constraints", meta = (DisplayName = "Group Boundary"))
	EGroupBoundaryConstraint GroupBoundaryConstraint = EGroupBoundaryConstraint::Ignore;

	/** Material Boundary Constraint Type */
	UPROPERTY(EditAnywhere, Category = "SimplifyMesh|Boundary Constraints", meta = (DisplayName = "Material Boundary"))
	EMaterialBoundaryConstraint MaterialBoundaryConstraint = EMaterialBoundaryConstraint::Ignore;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface
};

UENUM()
enum class EPlaneCutKeepSide : uint8
{
	Positive,
	Negative,
	Both
};

UCLASS(Experimental, Category = MeshOperation, Meta = (DisplayName="Plane Cut", ToolTip = "Experimental - Plane cut input meshes") )
class UDataprepPlaneCutOperation : public UDataprepEditingOperation
{
	GENERATED_BODY()

public:

	UDataprepPlaneCutOperation()
	{
		CutPlaneOrigin = FVector(0, 0, 0);
		CutPlaneKeepSide = EPlaneCutKeepSide::Positive;
		CutPlaneNormalAngles = FVector(0.0f, 0.0f, 0.0f);
		SpacingBetweenHalves = 1.0f;
		bFillCutHole = false;
		bExportSeparatePieces = true;
	}

	/** Origin of the cutting plane */
	UPROPERTY(EditAnywhere, Category = PlaneCut, meta = (DisplayName = "Plane's Origin"))
	FVector CutPlaneOrigin;

	/** Euler angles of the normal to the cutting plane (default plane is XY plane) */
	UPROPERTY(EditAnywhere, Category = PlaneCut, meta = (DisplayName = "Plane's Orientation"))
	FVector CutPlaneNormalAngles;

	/** Specify which section(s) of the cut to keep. If 'Keep Both' is selected, both sides of the 
		cut are computed and a new actor added with the negative side. 
	*/
	UPROPERTY(EditAnywhere, Category = PlaneCut, meta = (DisplayName = "Side(s) To Keep"))
	EPlaneCutKeepSide CutPlaneKeepSide;

	/** If keeping both halves, separate the two pieces by this amount */
	UPROPERTY(EditAnywhere, Category = PlaneCut, meta = (EditCondition = "CutPlaneKeepSide == EPlaneCutKeepSide::Both", UIMin = "0", ClampMin = "0"))
	float SpacingBetweenHalves;

	/** If true, the cut surface is filled with simple planar hole fill surface(s) */
	UPROPERTY(EditAnywhere, Category = PlaneCut, meta = (DisplayName = "Fill Holes"))
	bool bFillCutHole;

	/** If true, meshes cut into multiple pieces will be saved as separate assets. */
	UPROPERTY(EditAnywhere, Category = PlaneCut, meta = (DisplayName = "Export Separated Pieces As New Mesh Assets"))
	bool bExportSeparatePieces;

	//~ Begin UDataprepOperation Interface
public:
	virtual FText GetCategory_Implementation() const override
	{
		return FDataprepOperationCategories::MeshOperation;
	}

protected:
	virtual void OnExecution_Implementation(const FDataprepContext& InContext) override;
	//~ End UDataprepOperation Interface

private:
	/**
	 * Create new mesh component and assign new static mesh to it, and then add it to the Actor.
	 * If Actor is a AStaticMeshActor instance, then the new mesh component will be it's existing StaticMeshComponent, and we will assign the new static mesh to it.
	 * Otherwise, we create new StaticMeshComponent, assign NewStaticMesh, and set it as the Root Component of the Actor
	 */
	UStaticMeshComponent* FinalizeStaticMeshActor(
		AStaticMeshActor* InActor, 
		const FString& InMeshName,
		const FMeshDescription* InMeshDescription,
		int InNumMaterialSlots,
		const UStaticMesh* InOriginalMesh);

	TUniquePtr<FDynamicMesh3> CutStaticMesh(
		const FTransform& InTransform, 
		const UStaticMesh* InStaticMesh);

	void PerformCutting(
		bool bKeepOriginalMesh, 
		TArray<UStaticMesh*>& InStaticMeshes, 
		const TArray<FTransform>& InCutPlaneTransforms, 
		TArray<TArray<UStaticMeshComponent*>>& InReferencingComponentsToUpdate,
		TArray<UObject*>& OutModifiedStaticMeshes,
		TArray<UStaticMeshComponent*>& OutCutawayMeshes);

	// Borrowed from UPlaneCutOperatorFactory::MakeNewOperator
	TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator(
		const FTransform& InMeshLocalToWorld, 
		TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> InOriginalMesh,
		float InMeshUVScaleFactor);
};
