// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "DataprepGeometrySelectionTransforms.h"

class FDataprepGeometryOperationsModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};
  
IMPLEMENT_MODULE( FDataprepGeometryOperationsModule, DataprepGeometryOperations )
