// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataprepGeometrySelectionTransforms.h"
#include "Engine/Level.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/Texture.h"
#include "Engine/World.h"
#include "IMeshMergeUtilities.h"
#include "DataprepOperationsLibraryUtil.h"
#include "DataprepGeometryOperations.h"
#include "Materials/MaterialInstance.h"
#include "MeshDescriptionAdapter.h"
#include "MeshMergeModule.h"
#include "MeshMergeData.h"
#include "MeshAttributes.h"
#include "MeshMerge/MeshMergingSettings.h"
#include "Modules/ModuleManager.h"
#include "StaticMeshOperations.h"

#if WITH_PROXYLOD
#include "ProxyLODVolume.h"
#endif

#define LOCTEXT_NAMESPACE "DataprepGeometrySelectionTransforms"

namespace DataprepGeometryOperationsUtils
{
	void FindOverlappingActors( const TArray<AActor*>& InActorsToTest, const TArray<AActor*>& InActorsToTestAgainst, TArray<AActor*>& OutOverlappingActors, float InJacketingAccuracy, bool InCheckOverlapping )
	{
		if( InActorsToTestAgainst.Num() == 0 || InActorsToTest.Num() == 0 )
		{
			UE_LOGF( LogDataprepGeometryOperations, Warning, "FindOverlappingActors: No actors to process. Aborting..." );
			return;
		}

		TFunction<void(const TArray<AActor*>&, TArray<UStaticMeshComponent*>&, TMap<AActor*, int32>*)> GetActorsComponents = 
			[]( const TArray<AActor*>& InActors, TArray<UStaticMeshComponent*>& OutComponents, TMap<AActor*, int32>* OutActorComponentCounts )
		{
			for( AActor* Actor : InActors )
			{
				if( Actor == nullptr )
				{
					continue;
				}

				int32 ComponentCount = 0;
				for( UActorComponent* Component : Actor->GetComponents() )
				{
					if( UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component) )
					{
						if( StaticMeshComponent->GetStaticMesh() == nullptr )
						{
							continue;
						}
						ComponentCount++;
						OutComponents.Add( StaticMeshComponent );

						if( OutActorComponentCounts )
						{
							(*OutActorComponentCounts)[Actor] = ComponentCount;
						}
					}
				}
			}
		};

		TFunction<FMeshDescription*(UStaticMeshComponent*)> BakeMeshDescription = [](UStaticMeshComponent* InStaticMeshComponent) -> FMeshDescription*
		{
			UStaticMesh* StaticMesh = InStaticMeshComponent->GetStaticMesh();
			if (StaticMesh == nullptr)
			{
				return nullptr;
			}

			// FMeshMergeData will release the allocated MeshDescription...
			FMeshDescription* MeshDescriptionOriginal = StaticMesh->GetMeshDescription(0);
			FMeshDescription* MeshDescription = new FMeshDescription();
			*MeshDescription = *MeshDescriptionOriginal;
			//Make sure all ID are from 0 to N
			FElementIDRemappings OutRemappings;
			MeshDescription->Compact(OutRemappings);

			const FTransform& ComponentToWorldTransform = InStaticMeshComponent->GetComponentTransform();

			// Transform raw mesh vertex data by the Static Mesh Component's component to world transformation
			FStaticMeshOperations::ApplyTransform(*MeshDescription, ComponentToWorldTransform);

			return MeshDescription;
		};

		TSet< AActor* > InsideVolumeActorsSet;

#if WITH_PROXYLOD
		// Run jacketing to test for fully insdie actors (default)

		// Collect all StaticMeshComponent objects
		TArray<UStaticMeshComponent*> StaticMeshComponents;

		GetActorsComponents( InActorsToTestAgainst, StaticMeshComponents, nullptr );

		if (StaticMeshComponents.Num() == 0)
		{
			UE_LOGF( LogDataprepGeometryOperations, Warning, "FindOverlappingActors: No meshes to create a volume from. Aborting..." );
			return;
		}

		// Geometry input data for voxelizing methods
		TArray<FMeshMergeData> Geometry;
		// Store world space mesh for each static mesh component
		TMap<UStaticMeshComponent*, FMeshDescription*> MeshDescriptions;

		for( UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents )
		{
			FMeshDescription* MeshDescription = BakeMeshDescription( StaticMeshComponent );

			// Stores transformed MeshDescription for later use
			MeshDescriptions.Add(StaticMeshComponent, MeshDescription);

			FMeshMergeData MergeData;
			MergeData.bIsClippingMesh = false;
			MergeData.SourceStaticMesh = StaticMeshComponent->GetStaticMesh();
			MergeData.RawMesh = MeshDescription;

			Geometry.Add(MergeData);
		}

		if (Geometry.Num() == 0)
		{
			UE_LOGF( LogDataprepGeometryOperations, Warning, "FindOverlappingActors: No geometry to process. Aborting...");
			return;
		}

		TUniquePtr<IProxyLODVolume> Volume(IProxyLODVolume::CreateSDFVolumeFromMeshArray(Geometry, InJacketingAccuracy));
		if (!Volume.IsValid())
		{
			UE_LOGF( LogDataprepGeometryOperations, Error, "FindOverlappingActors: Voxelization of geometry failed. Aborting process..." );
			return;
		}

		double HoleRadius = 0.5 * InJacketingAccuracy;
		IProxyLODVolume::FVector3i VolumeBBoxSize = Volume->GetBBoxSize();

		// Clamp the hole radius.
		const double VoxelSize = Volume->GetVoxelSize();
		int32 MinIndex = VolumeBBoxSize.MinIndex();
		double BBoxMinorAxis = VolumeBBoxSize[MinIndex] * VoxelSize;
		if (HoleRadius > .5 * BBoxMinorAxis)
		{
			HoleRadius = .5 * BBoxMinorAxis;
			UE_LOGF( LogDataprepGeometryOperations, Warning, "FindOverlappingActors: Merge distance %f too large, clamped to %f.", InJacketingAccuracy, float(2. * HoleRadius) );
		}

		// Used in gap-closing.  This max is to bound a potentially expensive computation.
		// If the gap size requires more dilation steps at the current voxel size,
		// then the dilation (and erosion) will be done with larger voxels.
		const int32 MaxDilationSteps = 7;

		if (HoleRadius > 0.25 * VoxelSize && MaxDilationSteps > 0)
		{
			// Performance tuning number.  if more dilations are required for this hole radius, a coarser grid is used.
			Volume->CloseGaps(HoleRadius, MaxDilationSteps);
		}

		InsideVolumeActorsSet.Reserve(Geometry.Num());

		int32 OccludedComponentCount = 0;
		// Set the maximum distance over which a point is considered outside the volume
		// Set to twice the precision requested
		float MaxDistance = -2.0f * InJacketingAccuracy;

		TArray<UStaticMeshComponent*> ComponentsToTest;

		TMap<AActor*, int32> ActorOccurences;
		ActorOccurences.Reserve(InActorsToTest.Num());
		for (AActor* Actor : InActorsToTest)
		{
			ActorOccurences.Add(Actor, 0);
		}

		GetActorsComponents(InActorsToTest, ComponentsToTest, &ActorOccurences);

		if( ComponentsToTest.Num() == 0 )
		{
			UE_LOGF(LogDataprepGeometryOperations, Warning, "FindOverlappingActors: No meshes to process. Aborting...");
			return;
		}

		// Lazy init of mesh descriptions, only when needed
		TArray<FMeshDescription*> TestedMeshDescriptions;

		for (UStaticMeshComponent* StaticMeshComponent : ComponentsToTest)
		{
			FVector Min, Max;
			StaticMeshComponent->GetLocalBounds(Min, Max);

			bool bComponentInside = true;

			// Check the corners of the component's bounding box
			for (int32 i = 0; i < 8; i++)
			{
				FVector Corner = Min;
				if (i % 2)
				{
					Corner.X += Max.X - Min.X;
				}
				if ((i / 2) % 2)
				{
					Corner.Y += Max.Y - Min.Y;
				}
				if (i / 4)
				{
					Corner.Z += Max.Z - Min.Z;
				}
				const FTransform& ComponentTransform = StaticMeshComponent->GetComponentTransform();
				const FVector WorldCorner = ComponentTransform.TransformPosition(Corner);
				const float Value = Volume->QueryDistance(WorldCorner);

				if (Value > MaxDistance)
				{
					bComponentInside = false;
					break;
				}
			}

			// Component's bounding box intersect with volume, check on vertices
			if (!bComponentInside)
			{
				bComponentInside = true;

				FMeshDescription* MeshDescription = BakeMeshDescription( StaticMeshComponent );
				TestedMeshDescriptions.Add(MeshDescription);

				TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescription->VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
				for (FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
				{
					if (Volume->QueryDistance((FVector)VertexPositions[VertexID]) > MaxDistance)
					{
						bComponentInside = false;
						break;
					}
				}
			}

			if (bComponentInside)
			{
				OccludedComponentCount++;

				AActor* Actor = StaticMeshComponent->GetOwner();

				// Decrement number of
				int32& ComponentCount = ActorOccurences[Actor];
				--ComponentCount;

				// All static mesh components of an actor are occluded, take action
				if (ComponentCount == 0)
				{
					InsideVolumeActorsSet.Add(Actor);
				}
			}
		}

		// Free memory
		for (FMeshDescription* TempMeshDescription : TestedMeshDescriptions )
		{
			delete TempMeshDescription;
		}
#endif // WITH_PROXYLOD

		OutOverlappingActors = InsideVolumeActorsSet.Array();

		if (!InCheckOverlapping) 
		{
			return; // Not interested in overlapping actors, just fully in-volume
		}

		// Proceed with overlaping tests

		TArray<AActor*> ActorsNotInVolume;

		for (AActor* Actor : InActorsToTest)
		{
			if (!InsideVolumeActorsSet.Contains(Actor))
			{
				ActorsNotInVolume.Add(Actor);
			}
		}

		if (ActorsNotInVolume.Num() == 0)
		{
			return; // No actors left to test for overlapping
		}

		// Create merged mesh for target volume
		TArray<UStaticMeshComponent*> ComponentsToMerge;

		GetActorsComponents( InActorsToTestAgainst, ComponentsToMerge, nullptr );

		if( ComponentsToMerge.Num() == 0 )
		{
			UE_LOGF(LogDataprepGeometryOperations, Warning, "FindOverlappingActors: No meshes to process. Aborting...");
			return;
		}

		TSet<UStaticMesh*> StaticMeshes;
		TArray<UPrimitiveComponent*> PrimitiveComponentsToMerge; // because of MergeComponentsToStaticMesh
		for(UStaticMeshComponent* StaticMeshComponent : ComponentsToMerge )
		{
			PrimitiveComponentsToMerge.Add(StaticMeshComponent);

			if( StaticMeshComponent->GetStaticMesh()->GetRenderData() == nullptr )
			{
				StaticMeshes.Add( StaticMeshComponent->GetStaticMesh() );
			}
		}

		DataprepOperationsLibraryUtil::FStaticMeshBuilder StaticMeshBuilder( StaticMeshes );

		const IMeshMergeUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>( "MeshMergeUtilities" ).GetUtilities();

		FMeshMergingSettings MergeSettings;
		FVector MergedMeshWorldLocation;
		TArray<UObject*> CreatedAssets;
		const float ScreenAreaSize = TNumericLimits<float>::Max();
	
		MeshUtilities.MergeComponentsToStaticMesh( PrimitiveComponentsToMerge, nullptr, MergeSettings, nullptr, GetTransientPackage(), FString(), CreatedAssets, MergedMeshWorldLocation, ScreenAreaSize, true );

		UStaticMesh* MergedMesh = nullptr;
		if( !CreatedAssets.FindItemByClass( &MergedMesh ) )
		{
			UE_LOGF(LogDataprepGeometryOperations, Error, "MergeStaticMeshActors failed. No mesh was created.");
			return;
		}

		// Transform raw mesh vertex data by the Static Mesh Component's component to world transformation
		FStaticMeshOperations::ApplyTransform( *MergedMesh->GetMeshDescription(0), FTransform(MergedMeshWorldLocation) );

		// Buil mesh tree to test intersections
		FMeshDescriptionTriangleMeshAdapter MergedMeshAdapter( MergedMesh->GetMeshDescription(0) );
		UE::Geometry::TMeshAABBTree3 <FMeshDescriptionTriangleMeshAdapter> MergedMeshTree(&MergedMeshAdapter);

		MergedMeshTree.Build();

		check( MergedMeshTree.IsValid(false) );

		using FAxisAlignedBox3d = UE::Geometry::FAxisAlignedBox3d;
		const FAxisAlignedBox3d MergedMeshBox = MergedMeshTree.GetBoundingBox();

		TSet< AActor* > OverlappingActorSet;
		OverlappingActorSet.Reserve( ComponentsToMerge.Num() );

		TArray<UStaticMeshComponent*> StaticMeshComponentsToTest;
		GetActorsComponents( ActorsNotInVolume, StaticMeshComponentsToTest, nullptr );

		// Check each actor agains volume
		for( UStaticMeshComponent* StaticMeshComponent : StaticMeshComponentsToTest )
		{
			const FAxisAlignedBox3d MeshBox( StaticMeshComponent->Bounds.GetBox() );
			bool bOverlap = MeshBox.Intersects( MergedMeshBox );

			if( bOverlap )
			{
				// Component's bounding box intersects with volume, check on vertices
				bOverlap = false;

				if( const UStaticMesh* Mesh = StaticMeshComponent->GetStaticMesh() )
				{
					const FMeshDescriptionTriangleMeshAdapter MeshAdapter( Mesh->GetMeshDescription(0) );
					const FTransform& ComponentTransform = StaticMeshComponent->GetComponentTransform();

					bOverlap = MergedMeshTree.TestIntersection( &MeshAdapter, FAxisAlignedBox3d::Empty(), [&ComponentTransform]( const FVector3d& InVert ) -> FVector3d
					{
						return ComponentTransform.TransformPosition( FVector( InVert.X, InVert.Y, InVert.Z ) );
					});

					if( bOverlap )
					{
						AActor* Actor = StaticMeshComponent->GetOwner();
						OverlappingActorSet.Add( Actor );
					}
				}
			}
		}

		OutOverlappingActors.Append( OverlappingActorSet.Array() );
	}
}

void UDataprepOverlappingActorsSelectionTransform::OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects)
{
	TSet<AActor*> TargetActors;
	UWorld* World = nullptr;

	for (UObject* Object : InObjects)
	{
		if (!ensure(Object) || !IsValidChecked(Object))
		{
			continue;
		}

		if (AActor* Actor = Cast< AActor >(Object))
		{
			if (World == nullptr)
			{
				World = Actor->GetWorld();
			}
			TargetActors.Add(Actor);
		}
	}

	if (World == nullptr || TargetActors.Num() == 0)
	{
		return;
	}

	TArray<AActor*> WorldActors;

	// Get all world actors that we want to test against our input set.
	for (ULevel* Level : World->GetLevels())
	{
		for (AActor* Actor : Level->Actors)
		{
			if (Actor)
			{
				// Skip actors that are present in the input.
				if (TargetActors.Contains(Actor) || !IsValidChecked(Actor) || Actor->IsUnreachable())
				{
					continue;
				}
				WorldActors.Add(Actor);
			}
		}
	}

	// Run the overlap test.
	TArray<AActor*> OverlappingActors;

	DataprepGeometryOperationsUtils::FindOverlappingActors(WorldActors, TargetActors.Array(), OverlappingActors, JacketingAccuracy, bSelectOverlapping );

	OutObjects.Append(OverlappingActors);

	if( bOutputCanIncludeInput )
	{
		OutObjects.Append( TargetActors.Array() );
	}
}

#undef LOCTEXT_NAMESPACE
