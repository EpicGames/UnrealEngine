// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SelectionSystem/DataprepSelectionTransform.h"

#include "CoreMinimal.h"
#include "IDetailCustomization.h"
#include "Delegates/DelegateCombinations.h"
#include "DetailLayoutBuilder.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/TextProperty.h"

#include "DataprepGeometrySelectionTransforms.generated.h"

UCLASS(Category = SelectionTransform, Meta = (DisplayName = "Select In Volume", ToolTip = "Return all actors overlapping the selected actors"))
class UDataprepOverlappingActorsSelectionTransform : public UDataprepSelectionTransform
{
	GENERATED_BODY()

protected:
	virtual void OnExecution_Implementation(const TArray<UObject*>& InObjects, TArray<UObject*>& OutObjects) override;

private:
	// Accuracy of the distance field approximation
	UPROPERTY(EditAnywhere, Category = SelectionTransform, meta = (Units = cm, UIMin = "0.1", UIMax = "100", ClampMin = "0"))
	float JacketingAccuracy = 3.0f;

	// If checked, select fully inside + overlapping actors. Else, select only actors that are fully inside.
	UPROPERTY(EditAnywhere, Category = SelectionTransform)
	bool bSelectOverlapping = false;
};
