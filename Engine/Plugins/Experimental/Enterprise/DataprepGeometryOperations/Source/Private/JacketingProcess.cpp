// Copyright Epic Games, Inc. All Rights Reserved.

#include "JacketingProcess.h"

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "Async/ParallelFor.h"
#include "Editor.h"
#include "Editor/TransBuffer.h"
#include "Editor/UnrealEdEngine.h"
#include "Engine/StaticMesh.h"
#include "Layers/LayersSubsystem.h"
#include "Logging/LogMacros.h"
#include "Math/Box.h"
#include "MeshMergeData.h"
#include "StaticMeshAttributes.h"
#include "Misc/ScopedSlowTask.h"
#include "StaticMeshOperations.h"
#include "Templates/Tuple.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/StrongObjectPtr.h"
#include "Components/StaticMeshComponent.h"

#if WITH_PROXYLOD
#include "ProxyLODVolume.h"
#endif
#include "Subsystems/AssetEditorSubsystem.h"


DEFINE_LOG_CATEGORY_STATIC(LogJacketingProcess, Display, All);

#define LOCTEXT_NAMESPACE "JacketingProcess"
//#define DEBUG_EXPORT_ENVELOP

namespace JacketingProcessUtils
{
	/**
	 * Roughly (gu)estimate if value of Options->Accuracy could generate huge number of voxels
	 * If so, warn user, choose a more adequate value and proceed
	 */
	float ValidateVoxelSize(float InAccuracy, const TArray<UStaticMeshComponent*>& StaticMeshComponents)
	{
		const uint64 EmpiricalReasonableVoxelCount = 1073741824;

		float OutAccuracy = InAccuracy;

		double TotalVolume = 0;
		for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
		{
			const FVector Scale = StaticMeshComponent->GetComponentTransform().GetScale3D();
			FVector ComponentExtent = StaticMeshComponent->GetStaticMesh()->GetBoundingBox().GetExtent() * Scale;

			// Add volume of component's bounding box
			ComponentExtent = ComponentExtent.ComponentMax(FVector(InAccuracy));
			TotalVolume += (double)ComponentExtent[0] * (double)ComponentExtent[1] * (double)ComponentExtent[2];
		}

		const uint64 MaxGuestimatedVoxelCount = (uint64) (TotalVolume / (InAccuracy*InAccuracy*InAccuracy));

		if (MaxGuestimatedVoxelCount > EmpiricalReasonableVoxelCount)
		{
			// Best guest on accuracy is to be equal to the value to use in order to potentially
			// subdivide the total volume of the selected actors in 'EmpiricalReasonableVoxelCount' cells.
			OutAccuracy = FMath::Pow(TotalVolume / (double)EmpiricalReasonableVoxelCount, 1.0/3.0);

			// Inform user accuracy has been modified
			UE_LOGF(LogJacketingProcess, Warning, "AssemblyJacketing: Voxel precision of %.5f too small. Using %.5f instead.", InAccuracy, OutAccuracy);
		}

		return OutAccuracy;
	}
}

#if WITH_PROXYLOD

void FJacketingProcess::FindOverlappingActors(const TArray<AActor*>& InActorsToTest, const TArray<AActor*>& InActorsToTestAgainst, const FJacketingOptions* Options, TArray<AActor*>& OutOverlappingActors, bool bSilent)
{
	if (InActorsToTestAgainst.Num() == 0 || InActorsToTest.Num() == 0)
	{
		UE_LOGF(LogJacketingProcess, Warning, "FindOverlappingActors: No actors to process. Aborting...");
		return;
	}

	// Collect start time to log amount of time spent to import incoming file
	uint64 StartTime = FPlatformTime::Cycles64();
	uint64 LastTime = FPlatformTime::Cycles64();

	TFunction<void(const TArray<AActor*>&, TArray<UStaticMeshComponent*>&)> GetActorsComponents = [](const TArray<AActor*>& InActors, TArray<UStaticMeshComponent*>& OutComponents)
	{
		for (AActor* Actor : InActors)
		{
			if (Actor == nullptr)
			{
				continue;
			}

			for (UActorComponent* Component : Actor->GetComponents())
			{
				if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
				{
					if (StaticMeshComponent->GetStaticMesh() == nullptr)
					{
						continue;
					}
					OutComponents.Add(StaticMeshComponent);
				}
			}
		}
	};

	TArray<UStaticMeshComponent*> BuildVolumeComponents;

	GetActorsComponents(InActorsToTestAgainst, BuildVolumeComponents);

	if (BuildVolumeComponents.Num() == 0)
	{
		UE_LOGF(LogJacketingProcess, Warning, "FindOverlappingActors: No meshes to process. Aborting...");
		return;
	}

	// Geometry input data for voxelizing methods
	TArray<FMeshMergeData> Geometry;

	for (UStaticMeshComponent* StaticMeshComponent : BuildVolumeComponents)
	{
		UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
		if (StaticMesh == nullptr)
		{
			continue;
		}

		// FMeshMergeData will release the newly created MeshDescription
		FMeshDescription* MeshDescriptionOriginal = StaticMesh->GetMeshDescription(0);
		FMeshDescription* MeshDescription = new FMeshDescription();
		*MeshDescription = *MeshDescriptionOriginal;
		//Make sure all ID are from 0 to N
		FElementIDRemappings OutRemappings;
		MeshDescription->Compact(OutRemappings);

		const FTransform& ComponentToWorldTransform = StaticMeshComponent->GetComponentTransform();

		// Transform raw mesh vertex data by the Static Mesh Component's component to world transformation
		FStaticMeshOperations::ApplyTransform(*MeshDescription, ComponentToWorldTransform);

		// Geometry input data for voxelizing methods
		FMeshMergeData MergeData;
		MergeData.bIsClippingMesh = false;
		MergeData.SourceStaticMesh = StaticMesh;
		MergeData.RawMesh = MeshDescription;

		Geometry.Add(MergeData);
	}

	if (Geometry.Num() == 0)
	{
		UE_LOGF(LogJacketingProcess, Warning, "FindOverlappingActors: No geometry to process. Aborting...");
		return;
	}

	const float Accuracy = JacketingProcessUtils::ValidateVoxelSize(Options->Accuracy, BuildVolumeComponents);

	TSharedPtr<FScopedSlowTask> Progress;
	const float Voxelization = 40.f;
	const float GapFilling = 20.f;
	const float MeshTesting = 38.f;

	if (!bSilent)
	{
		Progress = TSharedPtr<FScopedSlowTask>(new FScopedSlowTask(100.0f, LOCTEXT("StartWork", "Occlusion culling ..."), true, *GWarn));
		Progress->MakeDialog(true);
		Progress->EnterProgressFrame(1.f, FText::FromString(TEXT("Analyzing meshes ...")));
	}

	if (Progress.IsValid())
	{
		Progress->EnterProgressFrame(Voxelization, FText::FromString(TEXT("Creating Voxelization ...")));
	}

	TUniquePtr<IProxyLODVolume> Volume(IProxyLODVolume::CreateSDFVolumeFromMeshArray(Geometry, Accuracy));
	if (!Volume.IsValid())
	{
		UE_LOGF(LogJacketingProcess, Error, "FindOverlappingActors: Voxelization of geometry failed. Aborting process...");
		return;
	}

	// Release the memory used by geometry (deletes the allocated mesh descriptions)
	Geometry.Empty();

	if (Progress.IsValid())
	{
		Progress->EnterProgressFrame(GapFilling, FText::FromString(TEXT("Closing gaps ...")));
	}

	double IntermediateTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastTime);
	UE_LOGF(LogJacketingProcess, Log, "FindOverlappingActors: Creation of volume took %.3f s.", IntermediateTime);
	LastTime = FPlatformTime::Cycles64();

	double HoleRadius = 0.5 * Options->MergeDistance; // cm
	IProxyLODVolume::FVector3i VolumeBBoxSize = Volume->GetBBoxSize();

	// Clamp the hole radius.
	const double VoxelSize = Volume->GetVoxelSize();
	int32 MinIndex = VolumeBBoxSize.MinIndex();
	double BBoxMinorAxis = VolumeBBoxSize[MinIndex] * VoxelSize;
	if (HoleRadius > 0.5 * BBoxMinorAxis)
	{
		HoleRadius = 0.5 * BBoxMinorAxis;
		UE_LOGF(LogJacketingProcess, Warning, "OverlappingActors: Merge distance %f too large, clamped to %f.", Options->MergeDistance, float(2. * HoleRadius));
	}

	// Used in gap-closing.  This max is to bound a potentially expensive computation.
	// If the gap size requires more dilation steps at the current voxel size,
	// then the dilation (and erosion) will be done with larger voxels.
	const int32 MaxDilationSteps = 7;

	LastTime = FPlatformTime::Cycles64();

	if (HoleRadius > 0.25 * VoxelSize && MaxDilationSteps > 0)
	{
		// performance tuning number.  if more dilations are required for this hole radius, a coarser grid is used.
		Volume->CloseGaps(HoleRadius, MaxDilationSteps);
	}

	IntermediateTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastTime);
	UE_LOGF(LogJacketingProcess, Log, "FindOverlappingActors: Closure of gaps took %.3f s.", IntermediateTime);
	LastTime = FPlatformTime::Cycles64();

	TSet< AActor* > OverlappingActorSet;
	OverlappingActorSet.Reserve(Geometry.Num());

	// Set the maximum distance over which a point is considered outside the volume
	const float MaxDistance = -1.99f * Accuracy;

	TArray<UStaticMeshComponent*> StaticMeshComponentsToTest;

	GetActorsComponents(InActorsToTest, StaticMeshComponentsToTest);

	float ProcessingStep = MeshTesting / (float)StaticMeshComponentsToTest.Num();

	// Compute volume bounding box for quick early testing
	FBox VolumeBoundingBox( EForceInit::ForceInit );

	for (AActor* VolumeActor : InActorsToTestAgainst)
	{
		VolumeBoundingBox += VolumeActor->GetComponentsBoundingBox();
	}
	
	// Check each actor agains volume
	for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponentsToTest)
	{
		if (Progress.IsValid())
		{
			Progress->EnterProgressFrame(ProcessingStep, FText::FromString(TEXT("Checking inclusion of meshes ...")));
		}

		const FBox OverlapBox = VolumeBoundingBox.Overlap( StaticMeshComponent->Bounds.GetBox() );

		bool bComponentInside = (OverlapBox.GetVolume() > 0);

		// Component's bounding box intersects with volume, check on vertices
		// (we need at least one vertex inside of volume)
		if (bComponentInside)
		{
			bComponentInside = false;

			const FMeshDescription* MeshDescription = StaticMeshComponent->GetStaticMesh()->GetMeshDescription(0);
			const FTransform& ComponentTransform = StaticMeshComponent->GetComponentTransform();
			TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescription->VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);

			for (FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
			{
				const FVector WorldPosition = ComponentTransform.TransformPosition((FVector)VertexPositions[VertexID]);
				if (Volume->QueryDistance(WorldPosition) <= MaxDistance)
				{
					bComponentInside = true;
					break;
				}
			}
		}

		if (bComponentInside)
		{
			AActor* Actor = StaticMeshComponent->GetOwner();
			OverlappingActorSet.Add(Actor);
		}
	}

	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

	int32 ElapsedMin = int32(ElapsedSeconds / 60.0);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;
	UE_LOGF(LogJacketingProcess, Log, "FindOverlappingActors: took %d min %.3f s. %d occluded actors out of %d", ElapsedMin, ElapsedSeconds, OverlappingActorSet.Num(), BuildVolumeComponents.Num());

	OutOverlappingActors = OverlappingActorSet.Array();
}

// See FVoxelizeMeshMerging::ProxyLOD
void FJacketingProcess::ApplyJacketingOnMeshActors(const TArray<AActor*>& Actors, const FJacketingOptions* Options, TArray<AActor*>& OccludedActorArray, bool bSilent)
{
	if (Actors.Num() == 0)
	{
		UE_LOGF(LogJacketingProcess, Warning, "AssemblyJacketing: No actors to process. Aborting...");
		return;
	}

	//JacketingProcessUtils::UpdateUndoBufferSize();

	// Collect start time to log amount of time spent to import incoming file
	uint64 StartTime = FPlatformTime::Cycles64();
	uint64 LastTime = FPlatformTime::Cycles64();

	TMap<AActor*, int32> ActorOccurences;
	ActorOccurences.Reserve(Actors.Num());
	for (AActor* Actor : Actors)
	{
		ActorOccurences.Add(Actor, 0);
	}

	// Collect all StaticMeshComponent objects
	TArray<UStaticMeshComponent*> StaticMeshComponents;
	for (AActor* Actor : Actors)
	{
		if (Actor == nullptr)
		{
			continue;
		}

		int32 ComponentCount = 0;
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
			{
				if (StaticMeshComponent->GetStaticMesh() == nullptr)
				{
					continue;
				}
				ComponentCount++;
				StaticMeshComponents.Add(StaticMeshComponent);
			}
		}

		ActorOccurences[Actor] = ComponentCount;
	}

	if (StaticMeshComponents.Num() == 0)
	{
		UE_LOGF(LogJacketingProcess, Warning, "AssemblyJacketing: No meshes to process. Aborting...");
		return;
	}

	float Accuracy = JacketingProcessUtils::ValidateVoxelSize(Options->Accuracy, StaticMeshComponents);

	TSharedPtr<FScopedSlowTask> Progress;
	const float Voxelization		= Options->Target == EJacketingTarget::Mesh ? 10.f : 40.f;
	const float GapFilling			= Options->Target == EJacketingTarget::Mesh ? 5.f : 20.f;
	const float MeshTesting			= Options->Target == EJacketingTarget::Mesh ? 10.f : 38.f;
	const float VertexTesting		= Options->Target == EJacketingTarget::Mesh ? 20.f : 0.f;
	const float TriangleDeletion	= Options->Target == EJacketingTarget::Mesh ? 20.f : 0.f;
	const float MeshBuilding		= Options->Target == EJacketingTarget::Mesh ? 33.f : 0.f;
	if (!bSilent)
	{
		Progress = TSharedPtr<FScopedSlowTask>(new FScopedSlowTask(100.0f, LOCTEXT("StartWork", "Occlusion culling ..."), true, *GWarn));
		Progress->MakeDialog(true);
		Progress->EnterProgressFrame(1.f, FText::FromString(TEXT("Analyzing meshes ...")));
	}

	// Geometry input data for voxelizing methods
	TArray<FMeshMergeData> Geometry;
	// Store world space mesh for each static mesh component
	TMap<UStaticMeshComponent*, FMeshDescription*> MeshDescriptions;
	int32 VertexCount = 0;
	int32 PolygonCount = 0;
	int32 DeletedPolygonCount = 0;

	for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
	{
		UStaticMesh* StaticMesh = StaticMeshComponent->GetStaticMesh();
		if (StaticMesh == nullptr)
		{
			continue;
		}

		// FMeshMergeData will release the allocated MeshDescription...
		FMeshDescription* MeshDescriptionOriginal = StaticMesh->GetMeshDescription(0);
		FMeshDescription* MeshDescription = new FMeshDescription();
		*MeshDescription = *MeshDescriptionOriginal;
		//Make sure all ID are from 0 to N
		FElementIDRemappings OutRemappings;
		MeshDescription->Compact(OutRemappings);

		const FTransform& ComponentToWorldTransform = StaticMeshComponent->GetComponentTransform();

		// Transform raw mesh vertex data by the Static Mesh Component's component to world transformation
		FStaticMeshOperations::ApplyTransform(*MeshDescription, ComponentToWorldTransform);

		VertexCount += MeshDescription->Vertices().Num();
		PolygonCount += MeshDescription->Polygons().Num();

		// Stores transformed MeshDescription for later use
		MeshDescriptions.Add(StaticMeshComponent, MeshDescription);

		FMeshMergeData MergeData;
		MergeData.bIsClippingMesh = false;
		MergeData.SourceStaticMesh = StaticMesh;
		MergeData.RawMesh = MeshDescription;

		Geometry.Add(MergeData);
	}

	if (Geometry.Num() == 0)
	{
		UE_LOGF(LogJacketingProcess, Warning, "AssemblyJacketing: No geometry to process. Aborting...");
		return;
	}

	if (Progress.IsValid())
	{
		Progress->EnterProgressFrame(Voxelization, FText::FromString(TEXT("Creating Voxelization ...")) );
	}

	UE_LOGF(LogJacketingProcess, Log, "AssemblyJacketing: Processing %i components, %i polygons", StaticMeshComponents.Num(), PolygonCount);

	TUniquePtr<IProxyLODVolume> Volume(IProxyLODVolume::CreateSDFVolumeFromMeshArray(Geometry, Accuracy));
	if (!Volume.IsValid())
	{
		UE_LOGF(LogJacketingProcess, Error, "AssemblyJacketing: Voxelization of geometry failed. Aborting process...");
		return;
	}

	if (Progress.IsValid())
	{
		Progress->EnterProgressFrame(GapFilling, FText::FromString(TEXT("Closing gaps ...")));
	}

	double IntermediateTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastTime);
	UE_LOGF(LogJacketingProcess, Log, "AssemblyJacketing: Creation of volume took %.3f s.", IntermediateTime);
	LastTime = FPlatformTime::Cycles64();

	double HoleRadius = 0.5 * Options->MergeDistance;
	IProxyLODVolume::FVector3i VolumeBBoxSize = Volume->GetBBoxSize();

	// Clamp the hole radius.
	const double VoxelSize = Volume->GetVoxelSize();
	int32 MinIndex = VolumeBBoxSize.MinIndex();
	double BBoxMinorAxis = VolumeBBoxSize[MinIndex] * VoxelSize;
	if (HoleRadius > .5 * BBoxMinorAxis)
	{
		HoleRadius = .5 * BBoxMinorAxis;
		UE_LOGF(LogJacketingProcess, Warning, "AssemblyJacketing: Merge distance %f too large, clamped to %f.", Options->MergeDistance, float(2. * HoleRadius));
	}

	// Used in gap-closing.  This max is to bound a potentially expensive computation.
	// If the gap size requires more dilation steps at the current voxel size,
	// then the dilation (and erosion) will be done with larger voxels.
	const int32 MaxDilationSteps = 7;

	LastTime = FPlatformTime::Cycles64();

	if (HoleRadius > 0.25 * VoxelSize && MaxDilationSteps > 0)
	{
		// performance tuning number.  if more dilations are required for this hole radius, a coarser grid is used.
		Volume->CloseGaps(HoleRadius, MaxDilationSteps);
	}

	TSet< AActor* > OccludedActorSet;
	OccludedActorSet.Reserve(Geometry.Num());

	IntermediateTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastTime);
	UE_LOGF(LogJacketingProcess, Log, "AssemblyJacketing: Closure of gaps took %.3f s.", IntermediateTime);
	LastTime = FPlatformTime::Cycles64();

	int32 OccludedComponentCount = 0;
	// Set the maximum distance over which a point is considered outside the volume
	// Set to twice the precision requested
	float MaxDistance = -2.0f * Accuracy;

	float ProcessingStep = MeshTesting / (float)StaticMeshComponents.Num();
	for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
	{
		if (Progress.IsValid())
		{
			Progress->EnterProgressFrame(ProcessingStep, FText::FromString(TEXT("Checking inclusion of meshes ...")));
		}

		FVector Min, Max;
		StaticMeshComponent->GetLocalBounds(Min, Max);

		bool bComponentInside = true;

		// Check the corners of the component's bounding box
		for (int32 i = 0; i < 8; i++)
		{
			FVector Corner = Min;
			if (i % 2)
			{
				Corner.X += Max.X - Min.X;
			}
			if ((i / 2) % 2)
			{
				Corner.Y += Max.Y - Min.Y;
			}
			if (i / 4)
			{
				Corner.Z += Max.Z - Min.Z;
			}
			const FTransform& ComponentTransform = StaticMeshComponent->GetComponentTransform();
			const FVector WorldCorner = ComponentTransform.TransformPosition(Corner);
			const float Value = Volume->QueryDistance(WorldCorner);

			if (Value > MaxDistance)
			{
				bComponentInside = false;
				break;
			}
		}

		// Component's bounding box intersect with volume, check on vertices
		if (!bComponentInside)
		{
			bComponentInside = true;
			FMeshDescription* MeshDescription = MeshDescriptions[StaticMeshComponent];
			TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescription->VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
			for (FVertexID VertexID : MeshDescription->Vertices().GetElementIDs())
			{
				if (Volume->QueryDistance((FVector)VertexPositions[VertexID]) > MaxDistance)
				{
					bComponentInside = false;
					break;
				}
			}
		}

		if (bComponentInside)
		{
			DeletedPolygonCount += MeshDescriptions[StaticMeshComponent]->Polygons().Num();
			OccludedComponentCount++;

			AActor* Actor = StaticMeshComponent->GetOwner();

			// Decrement number of
			int32& ComponentCount = ActorOccurences[Actor];
			--ComponentCount;

			// All static mesh components of an actor are occluded, take action
			if (ComponentCount == 0)
			{
				OccludedActorSet.Add(Actor);
			}
		}
	}

	// Fill up input array and return if target is only Level
	if (Options->Target == EJacketingTarget::Level)
	{
		double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

		int32 ElapsedMin = int32(ElapsedSeconds / 60.0);
		ElapsedSeconds -= 60.0 * (double)ElapsedMin;
		UE_LOGF(LogJacketingProcess, Log, "AssemblyJacketing: took %d min %.3f s. %d occluded actors out of %d", ElapsedMin, ElapsedSeconds, OccludedActorSet.Num(), StaticMeshComponents.Num());

		OccludedActorArray = OccludedActorSet.Array();

		return;
	}

	IntermediateTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastTime);
	UE_LOGF(LogJacketingProcess, Log, "AssemblyJacketing: Processing of meshes took %.3f s.", IntermediateTime);
	LastTime = FPlatformTime::Cycles64();

	// Proceed with triangle deletion
	if (Options->MergeDistance > VoxelSize)
	{
		// Expand interior narrow band to reach visible vertices discarded because of gap filling
		Volume->ExpandNarrowBand(VoxelSize, Options->MergeDistance);

		// Update MaxDistance to reflect expansion of narrow band
		if (Options->MergeDistance > 2.0f * VoxelSize)
		{
			MaxDistance = -Options->MergeDistance;
		}
	}

	if (Progress.IsValid())
	{
		Progress->EnterProgressFrame(VertexTesting, FText::FromString(TEXT("Checking occlusion of vertices ...")));
	}

	typedef TTuple<UStaticMeshComponent*, const FVector3f*, bool> FVertexData;
	TArray<FVertexData> VertexDataArray;
	VertexDataArray.Reserve(VertexCount);

	LastTime = FPlatformTime::Cycles64();

	// Mark any vertices of visible meshes which are outside the volume
	for (UStaticMeshComponent* StaticMeshComponent : StaticMeshComponents)
	{
		if (OccludedActorSet.Find(StaticMeshComponent->GetOwner()) != nullptr)
		{
			continue;
		}

		TVertexAttributesConstRef<FVector3f> VertexPositions = MeshDescriptions[StaticMeshComponent]->VertexAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Vertex::Position);
		for (FVertexID VertexID : MeshDescriptions[StaticMeshComponent]->Vertices().GetElementIDs())
		{
			VertexDataArray.Emplace(FVertexData(StaticMeshComponent, &VertexPositions[VertexID], false));
		}
	}

	VertexCount = VertexDataArray.Num();

	ParallelFor(VertexDataArray.Num(), [&](int32 Index) {
		FVertexData& VertexData = VertexDataArray[Index];

		if (!VertexData.Get<2>() && Volume->QueryDistance((FVector)*VertexData.Get<1>()) > MaxDistance)
		{
			VertexData.Get<2>() = true;
		}
	});

	IntermediateTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastTime);
	UE_LOGF(LogJacketingProcess, Log, "AssemblyJacketing: Processing of vertices took %.3f s.", IntermediateTime);
	LastTime = FPlatformTime::Cycles64();

	if (Progress.IsValid())
	{
		Progress->EnterProgressFrame(TriangleDeletion, FText::FromString(TEXT("Deleting triangles ...")));
	}

	// Combining vertex visibility for meshes used in more than one static mesh component
	typedef TPair<UStaticMeshComponent*, TArray<bool>> TVisibilityPair;
	TMap<UStaticMesh*, TVisibilityPair> MeshesToRebuild;
	MeshesToRebuild.Reserve(StaticMeshComponents.Num());
	for (int32 Index = 0; Index < VertexCount; )
	{
		UStaticMeshComponent* StaticMeshComponent = VertexDataArray[Index].Get<0>();

		if (MeshesToRebuild.Find(StaticMeshComponent->GetStaticMesh()) == nullptr)
		{
			TVisibilityPair& VisibilityPair = MeshesToRebuild.Add(StaticMeshComponent->GetStaticMesh(), TVisibilityPair(StaticMeshComponent, TArray<bool>()));
			VisibilityPair.Value.AddZeroed(MeshDescriptions[StaticMeshComponent]->Vertices().Num());
		}

		TArray<bool>& VertexVisibility = MeshesToRebuild[StaticMeshComponent->GetStaticMesh()].Value;
		for (int32 VertexIndex = 0; Index < VertexCount && StaticMeshComponent == VertexDataArray[Index].Get<0>(); ++VertexIndex, ++Index)
		{
			VertexVisibility[VertexIndex] |= VertexDataArray[Index].Get<2>();
		}
	}

	// Removing occluded triangles from meshes
	TArray<UStaticMesh*> StaticMeshArray;
	MeshesToRebuild.GetKeys(StaticMeshArray);
	ParallelFor(MeshesToRebuild.Num(), [&](int32 Index) 
	{
		UStaticMesh* StaticMesh = StaticMeshArray[Index];
		TVisibilityPair& VisibilityPair = MeshesToRebuild[StaticMesh];
		UStaticMeshComponent* StaticMeshComponent = VisibilityPair.Key;
		TArray<bool>& VertexVisibility = VisibilityPair.Value;

		FMeshDescription* MeshDescription = MeshDescriptions[StaticMeshComponent];

		if (MeshDescription->Polygons().Num() == 0 || MeshDescription->Vertices().Num() == 0)
		{
			VisibilityPair.Key = nullptr;
			return;
		}

		FMeshDescription NewMeshDescription = *MeshDescription;

		TArray<FPolygonID> PolygonToRemove;
		for (const FPolygonID& PolygonID : NewMeshDescription.Polygons().GetElementIDs())
		{
			TArray<FVertexID> PolygonVertices;
			NewMeshDescription.GetPolygonVertices(PolygonID, PolygonVertices);
			bool bPolygonVisible = false;
			for (FVertexID VertexID : PolygonVertices)
			{
				bPolygonVisible |= VertexVisibility[VertexID.GetValue()];
			}
			if (!bPolygonVisible)
			{
				PolygonToRemove.Add(PolygonID);
			}
		}

		// All triangles are visible. This mesh might be instantiated in several locations
		if (PolygonToRemove.Num() == 0)
		{
			VisibilityPair.Key = nullptr;
		}
		// This should never happen. Such situation should have been caught earlier
		else if (PolygonToRemove.Num() == NewMeshDescription.Polygons().Num())
		{
			check(false);
		}
		// Update mesh to only contain visible triangles
		else
		{
			NewMeshDescription.DeletePolygons(PolygonToRemove);
			
			//Compact and Remap IDs so we have clean ID from 0 to n since we just erased some polygons
			FElementIDRemappings RemappingInfos;
			NewMeshDescription.Compact(RemappingInfos);

			FTransform Transform = StaticMeshComponent->GetComponentTransform().Inverse();

			//Apply the inverse component transform
			FStaticMeshOperations::ApplyTransform(NewMeshDescription, Transform);

			TVertexInstanceAttributesRef<FVector3f> VertexInstanceNormals = NewMeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Normal);
			TVertexInstanceAttributesRef<FVector3f> VertexInstanceTangents = NewMeshDescription.VertexInstanceAttributes().GetAttributesRef<FVector3f>(MeshAttribute::VertexInstance::Tangent);

			MeshDescription->Empty();

			//Create the missing normals and tangents on polygons because FStaticMeshOperations::ComputeTangentsAndNormals requires it
			if(!NewMeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Normal).IsValid()
				|| !NewMeshDescription.TriangleAttributes().GetAttributesRef<FVector3f>(MeshAttribute::Triangle::Tangent).IsValid())
			{
				FStaticMeshOperations::ComputeTriangleTangentsAndNormals(NewMeshDescription);
			}

			const FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
			EComputeNTBsFlags ComputeNTBsOptions = EComputeNTBsFlags::BlendOverlappingNormals;
			ComputeNTBsOptions |= BuildSettings.bRecomputeNormals ? EComputeNTBsFlags::Normals : EComputeNTBsFlags::None;
			ComputeNTBsOptions |= BuildSettings.bRecomputeTangents ? EComputeNTBsFlags::Tangents : EComputeNTBsFlags::None;
			ComputeNTBsOptions |= BuildSettings.bUseMikkTSpace ? EComputeNTBsFlags::UseMikkTSpace : EComputeNTBsFlags::None;
			ComputeNTBsOptions |= BuildSettings.bComputeWeightedNormals ? EComputeNTBsFlags::WeightedNTBs : EComputeNTBsFlags::None;
			ComputeNTBsOptions |= BuildSettings.bRemoveDegenerates ? EComputeNTBsFlags::IgnoreDegenerateTriangles : EComputeNTBsFlags::None;

			FStaticMeshOperations::ComputeTangentsAndNormals(NewMeshDescription, ComputeNTBsOptions);
			// TODO: Maybe add generation of lightmap UV here.

			// Update mesh description of static mesh with new geometry
			FMeshDescription* OrigMeshDescription = StaticMesh->GetMeshDescription(0);
			*OrigMeshDescription = MoveTemp(NewMeshDescription);
		}
	});

	for (UStaticMesh* StaticMesh : StaticMeshArray)
	{
		//Commit the result so the old FMeshDescription is updated
		StaticMesh->CommitMeshDescription(0);
	}

	IntermediateTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastTime);
	UE_LOGF(LogJacketingProcess, Log, "AssemblyJacketing: Deleting triangles took %.3f s.", IntermediateTime);
	LastTime = FPlatformTime::Cycles64();

	// Rebuilding meshes which have been truncated
	ProcessingStep = MeshBuilding / (float)MeshesToRebuild.Num();
	for (auto& MeshToRebuild : MeshesToRebuild)
	{
		UStaticMesh* StaticMesh = MeshToRebuild.Key;

		if (Progress.IsValid())
		{
			Progress->EnterProgressFrame(ProcessingStep, FText::FromString(TEXT("Building meshes ...")));
		}

		if (MeshToRebuild.Value.Key != nullptr)
		{
			FMeshBuildSettings CachedBuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
			FMeshBuildSettings& BuildSettings = StaticMesh->GetSourceModel(0).BuildSettings;
			BuildSettings.bRecomputeNormals = false;
			BuildSettings.bRecomputeTangents = false;
			StaticMesh->PreEditChange(nullptr);
			StaticMesh->CommitMeshDescription(0);
			StaticMesh->PostEditChange();
			BuildSettings.bRecomputeNormals = CachedBuildSettings.bRecomputeNormals;
			BuildSettings.bRecomputeTangents = CachedBuildSettings.bRecomputeTangents;
		}
	}

	IntermediateTime = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - LastTime);
	UE_LOGF(LogJacketingProcess, Log, "AssemblyJacketing: Rebuilding static meshes took %.3f s.", IntermediateTime);
	LastTime = FPlatformTime::Cycles64();

	if (Progress.IsValid())
	{
		Progress->EnterProgressFrame(1.f, FText::FromString(TEXT("Updating level ...")));
	}

	for (FMeshMergeData& MergeData : Geometry)
	{
		MergeData.ReleaseData();
	}

#if defined(DEBUG_EXPORT_ENVELOP)
	{
		FSoftObjectPath SoftObjectPath(TEXT("/Game/jacketing.jacketing"));
		UStaticMesh* VolumeStaticMesh = Cast<UStaticMesh>(SoftObjectPath.TryLoad());

		if (VolumeStaticMesh)
		{
			FMeshDescription* MeshDescription = VolumeStaticMesh->GetMeshDescription(0);
			Volume->ConvertToRawMesh(*MeshDescription);
			// Update raw mesh with new geometry
			VolumeStaticMesh->PreEditChange(nullptr);
			VolumeStaticMesh->CommitMeshDescription(0);
			VolumeStaticMesh->PostEditChange();
		}
	}
#endif

	// Delete actors
	if (OccludedActorSet.Num() > 0)
	{
		UWorld* World = GEditor->GetEditorWorldContext(false).World();
		ULayersSubsystem* LayersSubsystem = GEditor->GetEditorSubsystem<ULayersSubsystem>();

		for (AActor* Actor : OccludedActorSet)
		{
			//To avoid dangling gizmo after actor has been destroyed
			if (Actor->IsSelected())
			{
				GEditor->SelectActor(Actor, false, true);
			}

			LayersSubsystem->DisassociateActorFromLayers(Actor);

			if (!World->DestroyActor(Actor, false, true))
			{
				UE_LOGF(LogJacketingProcess, Error, "AssemblyJacketing: Cannot delete Actor %ls.", *Actor->GetActorLabel());
			}
		}

		World->BroadcastLevelsChanged();
	}

	// Log time spent to perform jacketing process
	double ElapsedSeconds = FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime);

	int32 ElapsedMin = int32(ElapsedSeconds / 60.0);
	ElapsedSeconds -= 60.0 * (double)ElapsedMin;
	UE_LOGF(LogJacketingProcess, Log, "AssemblyJacketing: took %d min %.3f s. %d occluded components out of %d, %d polygons deleted out of %d",
																ElapsedMin, ElapsedSeconds, OccludedComponentCount, StaticMeshComponents.Num(), DeletedPolygonCount, PolygonCount);
}
#else
void FJacketingProcess::ApplyJacketingOnMeshActors(const TArray<AActor*>&, const FJacketingOptions*, TArray<AActor*>&, bool) {}
void FJacketingProcess::FindOverlappingActors(const TArray<AActor*>& InActorsToTest, const TArray<AActor*>& InActorsToTestAgainst, const FJacketingOptions* Options, TArray<AActor*>& OutOverlappingActors, bool bSilent) {}
#endif // WITH_PROXYLOD


#undef LOCTEXT_NAMESPACE
