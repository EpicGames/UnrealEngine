// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "MeshDescription.h"

enum class EJacketingTarget : uint8
{
	/** Apply jacketing on the level, will hide/tag/destroy actors and static mesh components. */
	Level,
	/** Apply jacketing on the mesh, will remove triangles/vertices. */
	Mesh
};

class FJacketingOptions
{
public:
	FJacketingOptions() : Accuracy(1.0F), MergeDistance(0.0F), Target(EJacketingTarget::Level)
	{ }

	FJacketingOptions(float InAccuracy, float InMergeDistance, EJacketingTarget InTarget)
		: Accuracy(InAccuracy)
		, MergeDistance(InMergeDistance)
		, Target(InTarget)
	{ }

	/** Accuracy of the distance field approximation, in UE units. */
	float Accuracy;

	/** Merge distance used to fill gap, in UE units. */
	float MergeDistance;

	/** Target to apply the jacketing to. */
	EJacketingTarget Target;
};

class FJacketingProcess
{
public:
	static void ApplyJacketingOnMeshActors(const TArray<AActor*>& Actors, const FJacketingOptions* Options, TArray<AActor*>& OccludedActorArray, bool bSilent);
	/**
	* Find the set of all actors that overlap any of the actors in another set.
	* @param	InActorsToTest			Input actors to test for overlap.
	* @param	InActorsToTestAgainst	Input actors to test against. Each actor in InActorsToTest will be tested against the volume built from InActorsToTestAgainst.
	* @param	Options					Parameter values to use for the overlapping.
	* @param	OutOverlappingActors	The actors that pass the overlap test (a subset from InActorsToTest).
	* @param	bSilent					Enable/disable progress dialog.
	*/
	static void FindOverlappingActors(const TArray<AActor*>& InActorsToTest, const TArray<AActor*>& InActorsToTestAgainst, const FJacketingOptions* Options, TArray<AActor*>& OutOverlappingActors, bool bSilent);
};